// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GraphColor : ModuleRules
{
	public GraphColor(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(new string[] { "Core" });
	}
}
