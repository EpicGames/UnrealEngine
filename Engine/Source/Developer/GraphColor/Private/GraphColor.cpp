// Copyright Epic Games, Inc. All Rights Reserved.

#include "GraphColor.h"
#include "Modules/ModuleManager.h"

#include "graph.h"
#include "graphColorVertices.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, GraphColor);

void GraphColorMesh( TArray< uint8 >& VertColors, const TArray< uint32 >& Indexes )
{
	graphP Graph = gp_New();
	int r = gp_InitGraph( Graph, VertColors.Num() );

	for( int i = 0; i < Indexes.Num(); i += 3 )
	{
		for( int k = 0; k < 3; k++ )
		{
			uint32 k1 = (1 << k) & 3;

			int u = gp_GetFirstVertex( Graph ) + Indexes[ i + k ];
			int v = gp_GetFirstVertex( Graph ) + Indexes[ i + k1 ];

			if( !gp_IsNeighbor( Graph, u, v ) )
			{
				r = gp_AddEdge( Graph, u, 0, v, 0 );
				check( r == OK );
			}
		}
	}

	r = gp_ColorVertices( Graph );
	check( r == OK );

	gp_CopyColors( Graph, VertColors.GetData() );

	gp_Free( &Graph );
}