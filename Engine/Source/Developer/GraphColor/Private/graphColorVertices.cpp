/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#include "graphColorVertices.h"
#include "graphColorVertices.private.h"
#include "graph.h"

#include "CoreMinimal.h"
#include <stdlib.h>

extern int COLORVERTICES_ID;

extern void _ClearVertexVisitedFlags(graphP theGraph, int);
extern int  _TestSubgraph(graphP theSubgraph, graphP theGraph);

/* Private functions exported to system */

void _AddVertexToDegList(ColorVerticesContext *context, graphP theGraph, int v, int deg);
void _RemoveVertexFromDegList(ColorVerticesContext *context, graphP theGraph, int v, int deg);
int  _AssignColorToVertex(ColorVerticesContext *context, graphP theGraph, int v);

/* Private functions */

int _GetVertexToReduce(ColorVerticesContext *context, graphP theGraph);
int _IsConstantTimeContractible(ColorVerticesContext *context, int v);
int _GetContractibleNeighbors(ColorVerticesContext *context, int v, int *pu, int *pw);

/********************************************************************
 gp_ColorVertices()

 This is the entry point for requesting a vertex coloring by the
 the minimum degree selection method.

 The call pattern is to simply invoke this function on a graph to
 color it or recolor it after some mutations.  It will invoke
 gp_AttachColorVertices() to attach the auxiliary data needed to
 performing the coloring, and the attachment short-circuits if
 already done.

 After calling this function, call gp_ColorVertices_GetColors() to
 obtain the colors or gp_Write() to save the colors. To read a saved
 coloring, use gp_AttachColorVertices() then gp_Read().

 Returns OK on success, NOTOK on failure
 ********************************************************************/

int gp_ColorVertices(graphP theGraph)
{
	// Avoid false positive static analysis C6011 warning. Remove when it is fixed
	CA_ASSUME(theGraph);
	CA_ASSUME(theGraph->E);
	CA_ASSUME(theGraph->V);

	ColorVerticesContext *context = NULL;
    int v, deg;
    int u=0, w=0, contractible;

    // Attach the algorithm if it is not already attached
	if (gp_AttachColorVertices(theGraph) != OK)
		return NOTOK;

	// Ensure there is enough stack to perform this operation.
	// At a maximum, the graph reduction will push 7N+M integers.
	// One integer is pushed per edge that is hidden. Plus, whether
	// a vertex is hidden or identified with another vertex, 7 integers
	// are used to store enough information to restore it.
	if (sp_NonEmpty(theGraph->theStack))
		return NOTOK;

	if (sp_GetCapacity(theGraph->theStack) < 7*theGraph->N + theGraph->M)
	{
		stackP newStack = sp_New(7*theGraph->N + theGraph->M);
		if (newStack == NULL)
			return NOTOK;
		sp_Free(&theGraph->theStack);
		theGraph->theStack = newStack;
	}

	// Get the extension context and reinitialize it if necessary
    gp_FindExtension(theGraph, COLORVERTICES_ID, (void **)&context);

    //if (context->color[0] > -1)
    //	return NOTOK;

    // Initialize the degree lists, and provide a color for any trivial vertices
    for (v = gp_GetFirstVertex(theGraph); gp_VertexInRange(theGraph, v); v++)
    {
    	deg = gp_GetVertexDegree(theGraph, v);
    	_AddVertexToDegList(context, theGraph, v, deg);

    	if (deg == 0)
    		context->color[v] = 0;
    }

    // Initialize the vertex visited flags so they can be used during reductions
    _ClearVertexVisitedFlags(theGraph, FALSE);

    // Reduce the graph using minimum degree selection
    while (context->numVerticesToReduce > 0)
    {
    	v = _GetVertexToReduce(context, theGraph);

    	// Find out if v is contractible and the neighbors to contract
    	contractible = _GetContractibleNeighbors(context, v, &u, &w);

    	// Remove the vertex from the graph. This calls the fpHideEdge
    	// overload, which performs the correct _RemoveVertexFromDegList()
    	// and _AddVertexToDegList() operations on v and its neighbors.
    	if (gp_HideVertex(theGraph, v) != OK)
    		return NOTOK;

    	// If v was contractibile, then identify u and w
    	if (contractible)
    	{
    		if (gp_IdentifyVertices(theGraph, u, w, NIL) != OK)
    			return NOTOK;
    	}
    }

    // Restore the graph one vertex at a time, coloring each vertex distinctly
    // from its neighbors as it is restored.
    context->colorDetector = (int *) calloc(theGraph->N, sizeof(int));
    if (context->colorDetector == NULL)
    	return NOTOK;

    if (gp_RestoreVertices(theGraph) != OK)
    	return NOTOK;

    free(context->colorDetector);
    context->colorDetector = NULL;

	return OK;
}

/********************************************************************
 _AddVertexToDegList()

 This function adds vertex v to degree list deg.
 The current method simply appends the vertex to the degree list.

 This method will be improved later to handle the degree 5 list
 specially by prepending those degree 5 vertices that have two
 non-adjacent neighbors with a constant degree bound. These vertices
 can be specially handled by identifying the non-adjacent neighbors
 during reduction so that the neighborhood of v receives only three
 colors.  This ensures that all planar graphs use at most 5 colors.
 Matula, Shiloach and Tarjan (1980) introduced this contraction
 method, and the tighter degree bound on the neighbors used in this
 implementation is due to Frederickson (1984).
 ********************************************************************/

void _AddVertexToDegList(ColorVerticesContext *context, graphP theGraph, int v, int deg)
{
	if (deg > 0)
	{
		if (_IsConstantTimeContractible(context, v))
			context->degListHeads[deg] = LCPrepend(context->degLists, context->degListHeads[deg], v);
		else
			context->degListHeads[deg] = LCAppend(context->degLists, context->degListHeads[deg], v);

        context->numVerticesToReduce++;
	}
	context->degree[v] = deg;
}

/********************************************************************
 _GetVertexDegree()
 ********************************************************************/

int _GetVertexDegree(ColorVerticesContext *context, int v)
{
	return context->degree[v];

	// We cache vertex degree values because the API function is O(deg(v)),
	// which would make this algorithm implementation have quadratic behavior
	// in the worst case
	//
	//return gp_GetVertexDegree(context->theGraph, v);
}

/********************************************************************
 _IsConstantTimeContractible()
 Wrapper function that just returns the result of _GetContractibleNeighbors()
 Return TRUE if v is degree 5 and has a pair of non-adjacent neighbors
 of degree 7 or lower; FALSE otherwise.
 ********************************************************************/

int _IsConstantTimeContractible(ColorVerticesContext *context, int v)
{
	int u, w;
	return _GetContractibleNeighbors(context, v, &u, &w);
}

/********************************************************************
 _GetContractibleNeighbors()
 Wrapper function that just returns the result of _GetContractibleNeighbors()

 This function returns TRUE if the vertex v is degree 5 and has two
 non-adjacent neighbors of degree at most 7.  In 1980, Matula, Shiloach
 and Tarjan proved the sequential contraction method of five-coloring
 planar graphs could run in linear time based on deleting any vertices
 less than degree 5 and, if none exist, contracting a degree 5 vertex
 with two non-adjacent neighbors of degree at most 11.  In 1984,
 Greg N. Frederickson improved the result to 7.

 When a vertex is being added to the degree list, it is appended
 unless this function returns TRUE, in which case it is placed
 at the front of the degree 5 list.
 When a vertex is removed from a degree list for reduction, it is
 tested again, and if this function returns TRUE, then two non-adjacent
 neighbors of degree at most 7 are found. The vertex is hidden in
 either case, but if the neighbors were found, then they are
 identified.  In the recursion, the neighbors will get the same
 color so that when the vertex is restored, its neighborhood has at
 most four colors.  The vertex takes the fifth color.
 Hence, planar graphs are colored with at most five colors. Non-planar
 graphs are still colored, but perhaps with more than five colors since
 the 5 list may become empty or may not start with a constant time
 contractible vertex (in which case we stick with the constant time
 per edge deletion only).

 This function operates in constant time, so it only finds a pair of
 contractible neighbors for degree 5 vertices, it determines the degree
 of all neighbors in constant time, it determines whether each pair of
 low degree neighbors is non-adjacent in constant time, and the degree
 bound on the pair of neighbors returned ensures that they can be
 identified (including removal of duplicate edges) in constant time.

 Return TRUE if v is degree 5 and has a pair of non-adjacent neighbors
 of degree 7 or lower; FALSE otherwise.

 Also returns the two neighbors found if TRUE is returned. The pointer
 variables are not altered in the FALSE case.
 ********************************************************************/

int _GetContractibleNeighbors(ColorVerticesContext *context, int v, int *pu, int *pw)
{

	int lowDegreeNeighbors[5], i, j, n=0, e;
	graphP theGraph = context->theGraph;

	// Avoid false positive static analysis C6011 warning. Remove when it is fixed
	CA_ASSUME(theGraph);
	CA_ASSUME(theGraph->E);
	CA_ASSUME(theGraph->V);

	// This method is only applicable to degree 5 vertices
	if (_GetVertexDegree(context, v) != 5)
		return FALSE;
	if (gp_GetVertexDegree(context->theGraph, v) != 5)
		return FALSE;

	// Get all neighbors of degree at most 7
	for( e = gp_GetFirstArc(theGraph, v); gp_IsArc(e); e = gp_GetNextArc(theGraph, e) )
    {
    	if (_GetVertexDegree(context, gp_GetNeighbor(theGraph, e)) <= 7)
    		lowDegreeNeighbors[n++] = gp_GetNeighbor(theGraph, e);
    }

    // Seek the pair of *non-adjacent* low degree neighbors
    for (i=0; i < (n-1); i++)
    	for (j=i+1; j < n; j++)
    		if (!gp_IsNeighbor(theGraph, lowDegreeNeighbors[i], lowDegreeNeighbors[j]))
    		{
    			*pu = lowDegreeNeighbors[i];
    			*pw = lowDegreeNeighbors[j];
    			return TRUE;
    		}

    // The desired pair of neighbors was not found
    return FALSE;
}


/********************************************************************
 _RemoveVertexFromDegList()
 ********************************************************************/

void _RemoveVertexFromDegList(ColorVerticesContext *context, graphP theGraph, int v, int deg)
{
	if (deg > 0)
	{
		context->degListHeads[deg] = LCDelete(context->degLists, context->degListHeads[deg], v);
	    context->numVerticesToReduce--;
	}
}

/********************************************************************
 _GetVertexToReduce()
 ********************************************************************/

int _GetVertexToReduce(ColorVerticesContext *context, graphP theGraph)
{
	int v = NIL, deg;

	for (deg = 1; deg < theGraph->N; deg++)
	{
		if (gp_IsVertex(context->degListHeads[deg]))
		{
			// Get the first vertex in the list
			v = context->degListHeads[deg];
			break;
		}
	}

	return v;
}

/********************************************************************
 _AssignColorToVertex()
 ********************************************************************/

int _AssignColorToVertex(ColorVerticesContext *context, graphP theGraph, int v)
{
	// Avoid false positive static analysis C6011 warning. Remove when it is fixed
	CA_ASSUME(theGraph);
	CA_ASSUME(theGraph->E);
	CA_ASSUME(theGraph->V);

	int e, w, color;

	// Run the neighbor list of v and flag all the colors in use
    e = gp_GetFirstArc(theGraph, v);
    while (gp_IsArc(e))
    {
         w = gp_GetNeighbor(theGraph, e);
         context->colorDetector[context->color[w]] = 1;

         e = gp_GetNextArc(theGraph, e);
    }

    // Find the least numbered unused color and assign it to v
    // Note that this loop never runs more than deg(v) steps
    for (color = 0; color < theGraph->N; color++)
    {
        if (context->colorDetector[color] == 0)
        {
        	context->color[v] = color;
        	if (context->highestColorUsed < color)
        		context->highestColorUsed = color;
        	break;
        }
    }

    if (context->color[v] < 0)
    	return NOTOK;

    // Run the neighbor list of v and unflag all the colors in use
    e = gp_GetFirstArc(theGraph, v);
    while (gp_IsArc(e))
    {
         w = gp_GetNeighbor(theGraph, e);
         context->colorDetector[context->color[w]] = 0;

         e = gp_GetNextArc(theGraph, e);
    }

	return OK;
}

/********************************************************************
 gp_GetNumColorsUsed()
 ********************************************************************/

int gp_GetNumColorsUsed(graphP theGraph)
{
    ColorVerticesContext *context = (ColorVerticesContext *) gp_GetExtension(theGraph, COLORVERTICES_ID);
	return context == NULL ? 0 : context->highestColorUsed+1;
}



void gp_CopyColors(graphP theGraph, unsigned char* colors)
{
	// Avoid false positive static analysis C6011 warning. Remove when it is fixed
	CA_ASSUME(theGraph);
	CA_ASSUME(theGraph->E);
	CA_ASSUME(theGraph->V);

	ColorVerticesContext *context = (ColorVerticesContext *) gp_GetExtension(theGraph, COLORVERTICES_ID);
	
	for ( int v = gp_GetFirstVertex(theGraph); gp_VertexInRange(theGraph, v); v++)
	{
		int i = v - gp_GetFirstVertex(theGraph);
		colors[i] = (unsigned char)context->color[v];
	}
}