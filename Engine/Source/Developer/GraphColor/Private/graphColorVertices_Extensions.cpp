/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#include "CoreMinimal.h"

#include "graphColorVertices.private.h"
#include "graphColorVertices.h"

extern void _AddVertexToDegList(ColorVerticesContext *context, graphP theGraph, int v, int deg);
extern void _RemoveVertexFromDegList(ColorVerticesContext *context, graphP theGraph, int v, int deg);
extern int  _AssignColorToVertex(ColorVerticesContext *context, graphP theGraph, int v);
extern int _GetVertexDegree(ColorVerticesContext *context, int v);

/* Forward declarations of local functions */

void _ColorVertices_ClearStructures(ColorVerticesContext *context);
int  _ColorVertices_CreateStructures(ColorVerticesContext *context);
int  _ColorVertices_InitStructures(ColorVerticesContext *context);

/* Forward declarations of overloading functions */

int  _ColorVertices_InitGraph(graphP theGraph, int N);
void _ColorVertices_ReinitializeGraph(graphP theGraph);

int  _ColorVertices_ReadPostprocess(graphP theGraph, void *extraData, long extraDataSize);
int  _ColorVertices_WritePostprocess(graphP theGraph, void **pExtraData, long *pExtraDataSize);

void _ColorVertices_HideEdge(graphP theGraph, int e);
int  _ColorVertices_IdentifyVertices(graphP theGraph, int u, int v, int eBefore);
int  _ColorVertices_RestoreVertex(graphP theGraph);

/* Forward declarations of functions used by the extension system */

void *_ColorVertices_DupContext(void *pContext, void *theGraph);
void _ColorVertices_FreeContext(void *);

/****************************************************************************
 * COLORVERTICES_ID - the variable used to hold the integer identifier for this
 * extension, enabling this feature's extension context to be distinguished
 * from other features' extension contexts that may be attached to a graph.
 ****************************************************************************/

int COLORVERTICES_ID = 0;

/****************************************************************************
 gp_AttachColorVertices()

 This function adjusts the graph data structure to attach the graph
 vertex coloringfeature.

 To activate this feature, call gp_ColorVertices().

 This method may be called immediately after gp_New() in the case of
 invoking gp_Read().  For generating graphs, gp_InitGraph() can be invoked
 before or after this enabling method.  This method detects if the core
 graph has already been initialized, and if so, it will initialize the
 additional data structures specific to vertex coloring.  This makes
 it possible to invoke gp_New() and gp_InitGraph() together, and then attach
 this feature only if it is requested at run-time.

 Returns OK for success, NOTOK for failure.
 ****************************************************************************/

int  gp_AttachColorVertices(graphP theGraph)
{
     ColorVerticesContext *context = NULL;

     // If the vertex coloring feature has already been attached to the graph,
     // then there is no need to attach it again
     gp_FindExtension(theGraph, COLORVERTICES_ID, (void **)&context);
     if (context != NULL)
     {
         return OK;
     }

     // Allocate a new extension context
     context = (ColorVerticesContext *) malloc(sizeof(ColorVerticesContext));
     if (context == NULL)
     {
         return NOTOK;
     }

     // First, tell the context that it is not initialized
     context->initialized = 0;

     // Save a pointer to theGraph in the context
     context->theGraph = theGraph;

     // Put the overload functions into the context function table.
     // gp_AddExtension will overload the graph's functions with these, and
     // return the base function pointers in the context function table
     memset(&context->functions, 0, sizeof(graphFunctionTable));

     context->functions.fpInitGraph = _ColorVertices_InitGraph;

     context->functions.fpHideEdge = _ColorVertices_HideEdge;
     context->functions.fpIdentifyVertices = _ColorVertices_IdentifyVertices;
     context->functions.fpRestoreVertex = _ColorVertices_RestoreVertex;

     _ColorVertices_ClearStructures(context);

     // Store the context, including the data structure and the
     // function pointers, as an extension of the graph
     if (gp_AddExtension(theGraph, &COLORVERTICES_ID, (void **) context,
                         _ColorVertices_DupContext, _ColorVertices_FreeContext,
                         &context->functions) != OK)
     {
         _ColorVertices_FreeContext(context);
         return NOTOK;
     }

     // Create the algorithm-specific structures if the size of the graph is known
     // Attach functions are typically invoked after gp_New(), but if a graph
     // extension must be attached before gp_Read(), then the attachment
     // also happens before gp_InitGraph() because gp_Read() invokes init only
     // after it reads the order N of the graph.  Hence, this attach call would
     // occur when N==0 in the case of gp_Read().
     // But if a feature is attached after gp_InitGraph(), then N > 0 and so we
     // need to create and initialize all the custom data structures
     if (theGraph->N > 0)
     {
         if (_ColorVertices_CreateStructures(context) != OK ||
             _ColorVertices_InitStructures(context) != OK)
         {
             _ColorVertices_FreeContext(context);
             return NOTOK;
         }
     }

     return OK;
}

/********************************************************************
 gp_DetachColorVertices()
 ********************************************************************/

int gp_DetachColorVertices(graphP theGraph)
{
    return gp_RemoveExtension(theGraph, COLORVERTICES_ID);
}

/********************************************************************
 _ColorVertices_ClearStructures()
 ********************************************************************/

void _ColorVertices_ClearStructures(ColorVerticesContext *context)
{
    if (!context->initialized)
    {
        // Before initialization, the pointers are stray, not NULL
        // Once NULL or allocated, free() or LCFree() can do the job
        context->degLists = NULL;
        context->degListHeads = NULL;
        context->degree = NULL;
        context->color = NULL;
        context->numVerticesToReduce = 0;
        context->highestColorUsed = -1;
        context->colorDetector = NULL;

        context->initialized = 1;
    }
    else
    {
        if (context->degLists != NULL)
        {
            LCFree(&context->degLists);
        }
        if (context->degListHeads != NULL)
        {
            free(context->degListHeads);
            context->degListHeads = NULL;
        }
        if (context->degree != NULL)
        {
            free(context->degree);
            context->degree = NULL;
        }
        if (context->color != NULL)
        {
            free(context->color);
            context->color = NULL;
        }
        context->numVerticesToReduce = 0;
        context->highestColorUsed = -1;
        context->colorDetector = NULL;
    }
}

/********************************************************************
 _ColorVertices_CreateStructures()
 Create uninitialized structures for the vertex and edge
 levels, and initialized structures for the graph level
 ********************************************************************/
int  _ColorVertices_CreateStructures(ColorVerticesContext *context)
{
	 graphP theGraph = context->theGraph;

     if (theGraph->N <= 0 || theGraph->N == INT_MAX)
	 {
         return NOTOK;
	 }

	 int VIsize = gp_PrimaryVertexIndexBound(theGraph);
     int v;

     if ((context->degLists = LCNew(VIsize)) == NULL ||
    	 (context->degListHeads = (int *) malloc(VIsize*sizeof(int))) == NULL ||
    	 (context->degree = (int *) malloc(VIsize*sizeof(int))) == NULL ||
         (context->color = (int *) malloc(VIsize*sizeof(int))) == NULL
        )
     {
         return NOTOK;
     }

	 // Static analysis workaround
	 check(VIsize == gp_GetFirstVertex(theGraph) + theGraph->N);

     for (v = gp_GetFirstVertex(theGraph); v < VIsize; v++)
     {
    	 context->degListHeads[v] = NIL;
    	 context->degree[v] = 0;
    	 context->color[v] = 0;
     }

     context->numVerticesToReduce = 0;
     context->highestColorUsed = -1;
     context->colorDetector = NULL;

     return OK;
}

/********************************************************************
 _ColorVertices_InitStructures()
 Intended to be called when N>0.
 Initializes vertex and edge levels only.  Graph level is
 initialized by _CreateStructures().
 ********************************************************************/
int  _ColorVertices_InitStructures(ColorVerticesContext *context)
{
     return OK;
}

/********************************************************************
 _ColorVertices_DupContext()
 ********************************************************************/

void *_ColorVertices_DupContext(void *pContext, void *pGraph)
{
     ColorVerticesContext *context = (ColorVerticesContext *) pContext;
     ColorVerticesContext *newContext = (ColorVerticesContext *) malloc(sizeof(ColorVerticesContext));
     graphP theGraph = (graphP) pGraph;

     if (newContext != NULL)
     {
         *newContext = *context;
         newContext->theGraph = theGraph;

         newContext->initialized = 0;
         _ColorVertices_ClearStructures(newContext);
         if (theGraph->N > 0)
         {
        	 int v;

             if (_ColorVertices_CreateStructures(newContext) != OK)
             {
                 _ColorVertices_FreeContext(newContext);
                 return NULL;
             }

             // Initialize custom data structures by copying
             LCCopy(newContext->degLists, context->degLists);
             for (v = gp_GetFirstVertex(theGraph); gp_VertexInRange(theGraph, v); v++)
             {
            	 newContext->degListHeads[v] = context->degListHeads[v];
            	 newContext->degree[v] = context->degree[v];
            	 newContext->color[v] = context->color[v];
             }
             newContext->numVerticesToReduce = context->numVerticesToReduce;
             newContext->highestColorUsed = context->highestColorUsed;
             newContext->colorDetector = NULL;
         }
     }

     return newContext;
}

/********************************************************************
 _ColorVertices_FreeContext()
 ********************************************************************/

void _ColorVertices_FreeContext(void *pContext)
{
     ColorVerticesContext *context = (ColorVerticesContext *) pContext;

     _ColorVertices_ClearStructures(context);
     free(pContext);
}

/********************************************************************
 ********************************************************************/

int  _ColorVertices_InitGraph(graphP theGraph, int N)
{
    ColorVerticesContext *context = NULL;
    gp_FindExtension(theGraph, COLORVERTICES_ID, (void **)&context);

    if (context == NULL)
	{
        return NOTOK;
	}

	theGraph->N = N;
	if (theGraph->arcCapacity == 0)
		theGraph->arcCapacity = 2*DEFAULT_EDGE_LIMIT*N;

	if (_ColorVertices_CreateStructures(context) != OK ||
        _ColorVertices_InitStructures(context) != OK)
		return NOTOK;

	context->functions.fpInitGraph(theGraph, N);

    return OK;
}

/********************************************************************
 _ColorVertices_HideEdge()

 An overload to perform the degree list updates for the edge endpoints.
 This routine also covers the work done by _HideVertex() and part of
 the work done by _ContractEdge() and _IdentifyVertices().
 ********************************************************************/
void _ColorVertices_HideEdge(graphP theGraph, int e)
{
    ColorVerticesContext *context = (ColorVerticesContext *) gp_GetExtension(theGraph, COLORVERTICES_ID);

    if (context != NULL)
    {
    	int u, v, udeg, vdeg;

    	// Get the endpoint vertices of the edge
    	u = gp_GetNeighbor(theGraph, e);
    	v = gp_GetNeighbor(theGraph, gp_GetTwinArc(theGraph, e));

    	// Get the degrees of the vertices
    	udeg = _GetVertexDegree(context, u);
    	vdeg = _GetVertexDegree(context, v);

    	// Remove them from the degree lists that contain them
    	_RemoveVertexFromDegList(context, theGraph, u, udeg);
    	_RemoveVertexFromDegList(context, theGraph, v, vdeg);

    	// Hide the edge
        context->functions.fpHideEdge(theGraph, e);

        // Decrement the degrees of the endpoint vertices
        udeg--;
        vdeg--;

        // Add them to the new degree lists
    	_AddVertexToDegList(context, theGraph, u, udeg);
    	_AddVertexToDegList(context, theGraph, v, vdeg);
    }
}

/********************************************************************
 _ColorVertices_IdentifyVertices()

 An overload to perform degree list updates corresponding to the
 transfer of v's adjacency list into u.  Since the common edges are
 removed using _HideEdge(), this routine only accounts for those
 edges of v that create new neighbors for u.  These are indicated on
 the top of the stack after the base routine is called.
 This routine also does part of the work of _ContractEdge(), which
 simply combines _HideEdge() and _IdentifyVertices().
 ********************************************************************/

int _ColorVertices_IdentifyVertices(graphP theGraph, int u, int v, int eBefore)
{
    ColorVerticesContext *context = (ColorVerticesContext *) gp_GetExtension(theGraph, COLORVERTICES_ID);

    if (context != NULL)
    {
    	int e_v_last, e_v_first;

    	// First, identify u and v.  No point in taking v's degree beforehand
    	// because some of its incident edges may indicate neighbors of u. This
    	// causes v to be moved to a lower degree list than deg(v).
        if (context->functions.fpIdentifyVertices(theGraph, u, v, eBefore) != OK)
            return NOTOK;

        // The edges transferred from v to u are indicated on the top of the
        // stack, which looks like this after identifying u and v:
        // ... e_u_succ e_v_last e_v_first e_u_pred u v
		e_v_first = sp_Get(theGraph->theStack, sp_GetCurrentSize(theGraph->theStack)-4);
		e_v_last = sp_Get(theGraph->theStack, sp_GetCurrentSize(theGraph->theStack)-5);

        // We count the number of edges K transferred from v to u after the
        // common edges were hidden
		if (gp_IsArc(e_v_first))
		{
			int e, K, degu;

			for (e=e_v_first, K=1; e != e_v_last; e=gp_GetNextArc(theGraph, e))
				K++;

	        // Remove v from the degree list K. During IdentifyVertices(), if v had any
			// common edges with u, they were "hidden", which reduced the degree of v to K.
			_RemoveVertexFromDegList(context, theGraph, v, K);

	        // We move u from degree list deg(u)-K to degree list deg(u)
			degu = gp_GetVertexDegree(theGraph, u);
			_RemoveVertexFromDegList(context, theGraph, u, degu-K);
			_AddVertexToDegList(context, theGraph, u, degu);
		}

        return OK;
    }

    return NOTOK;
}

/********************************************************************
 _ColorVertices_RestoreVertex()

 An overload to color the vertex distinctly from its neighbors once
 it is restored.
 ********************************************************************/

int _ColorVertices_RestoreVertex(graphP theGraph)
{
    ColorVerticesContext *context = (ColorVerticesContext *) gp_GetExtension(theGraph, COLORVERTICES_ID);

    if (context != NULL)
    {
    	int u, v;

    	// Read the stack to figure out which vertex is being restored
		u = sp_Get(theGraph->theStack, sp_GetCurrentSize(theGraph->theStack)-2);
		v = sp_Get(theGraph->theStack, sp_GetCurrentSize(theGraph->theStack)-1);

    	// Restore the vertex
        if (context->functions.fpRestoreVertex(theGraph) != OK)
            return NOTOK;

        // If the restored vertex v was hidden, then give it a color distinct from its neighbors
        // Note that u is not a vertex in this case
        if (gp_IsNotVertex(u))
        {
        	if (_AssignColorToVertex(context, theGraph, v) != OK)
        		return NOTOK;

        	if (context->color[v] < 0)
        		return NOTOK;
        }

        // Else if the restored vertex v was identified, then give v the same color as the
        // vertex u with which it was identified.
        else
        {
        	context->color[v] = context->color[u];
        }

        return OK;
    }

    return NOTOK;
}
