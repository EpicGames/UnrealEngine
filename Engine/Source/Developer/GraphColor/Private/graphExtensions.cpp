/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#include "graphExtensions.h"
#include "graphExtensions.private.h"

#include "CoreMinimal.h"
#include "appconst.h"

#include "graphFunctionTable.h"

/* Imported functions */

extern void _InitFunctionTable(graphP theGraph);

/* Private function */

void _FreeExtension(graphExtensionP extension);
void _OverloadFunctions(graphP theGraph, graphFunctionTableP functions);
void _FixupFunctionTables(graphP theGraph, graphExtensionP curr);
graphExtensionP _FindNearestOverload(graphP theGraph, graphExtensionP target, int functionIndex);

/********************************************************************
 * The moduleIDGenerator is used to help ensure that all extensions
 * added during a run-time have a different integer identifier.
 * An ID identifies an extension, which may be added to multiple
 * graphs.  It is used in lieu of identifying extensions by a string
 * name, which is noticeably expensive when a frequently called
 * overload function seeks the extension context for a graph.
 ********************************************************************/

static int moduleIDGenerator = 0;

/********************************************************************
 The extension mechanism allows new modules to equip a graph with the
 data structures and functions needed to implement new algorithms
 without impeding the performance of the core graph planar embedding
 algorithms on graphs that have not been so equipped.

 The following steps must be used to create a graph extension:

  1) Create a moduleID variable initialized to zero that will be
     assigned a positive integer the first time the extension is
     added to a graph by gp_AddExtension()

  2) Define an extension context structure to contain all of the data
     and function pointers that extend the graph.  The context must
     include a graphFunctionTable to allow overloading of functions.
     An instance of this context structure is passed to the "context"
     parameter of gp_AddExtension().

  3) Define a function capable of duplicating your context data
     structure.  It receives a void pointer indicating the context
     to duplicate and a void pointer that can be cast to a graph
     pointer indicating the graph for which the context is being
     duplicated.  The void pointer returned indicates the newly
     allocated context structure.  The pointer to this function is
     passed to the "dupContext" parameter of gp_AddExtension()

     Note: It is useful to store in your context structure a pointer
     to the graph that the context is extending.  There are certain
     function overloads you will perform that will only receive
     the context, and you may need to know things about the graph,
     such as the number of vertices or edges.

  4) Define a function that can free the memory used by your context
     data structure.  It will receive a void pointer indicating the
     instance of your context data structure that you passed as the
     "context" parameter to gp_AddExtension().
     The free function pointer should be passed as the "freeContext"
     parameter to gp_AddExtension()

  5) The expected method of attaching your feature to a graph is to
     create a function called gp_AttachFeature(), where 'Feature' is
     the name of your module.  The attach function allocates your context
     data structure, initializes the extension data, assigns overload
     function pointers, and invokes gp_AddExtension().

     NOTE: It is advisable to use memset on the context function table
     before assigning any function overloads because any function not
     being overloaded must have a NULL pointer.

     NOTE: The gp_AddExtension() method puts the overload function
     pointers into the graph's function table, and the base function
     pointers that were overloaded are placed in the context function
     table.  This allows the extension's functions to have access to
     base function behaviors, since many extension functions will
     extend rather than replace the base behavior.

  6) There are a few functions that you must overload in order to
     successfully manage data structures that are parallel to the
     main graph data structures.

     The core graph data structure has function pointers to functions
     that can be overloaded.  In addition to invoking gp_AddExtension(),
     you need to set pointers to your own versions of the functions
     you are overloading.  You will also need to store a copy of the
     prior pointer in your feature's context data structure so that you
     can invoke the "base" behavior from your function overload, e.g.
     if your feature is attached but not active or if your feature
     augments the base behavior rather than replacing it.

     a) If any kind of data structures needs to be maintained at
        the graph, vertex or edge levels, then an overload of
        fpInitGraph() will be needed.

     b) If any data must be associated with primary and virtual vertices,
        then an overload of fpInitVertexRec() is needed.  If data must be
        associated only with primary vertices (0 to N-1), then one can
        overload fpInitVertexInfo() instead.
        The overload function should be named _Feature_InitVertexRec()
        or _Feature_InitVertexInfo().
        It will invoke the base fpInitVertexRec() or fpInitVertexInfo()
        but then also invoke a second function named _InitFeatureVertexRec()
        or _InitFeatureVertexInfo() thatinitializes the custom VertexRec
        or VertexInfo data members.

     c) If any data must be associated with the edges, then an overload
        of fpInitEdgeRec() is needed.
        This overload function should be named _Feature_InitEdgeRec().
        It will invoke the base fpInitEdgeRec() and also invoke
        a second function named_InitFeatureEdgeRec() that
        initializes the custom EdgeRec data members

     d) If any graph-level data structures are needed, then an
        overload of fpReinitializeGraph() will also be needed, not just the
        overload of fpInitGraph().  However, if only vertex-level and/or
        edge level data members are needed, then the overloads of
        fpInitVertexRec(), fpInitVertexInfo() and/or fpInitEdgeRec() are
        invoked by the basic fpReinitializeGraph without needing to overload
        it as well.

     e) If any data must be persisted in the file format, then overloads
        of fpReadPostprocess() and fpWritePostprocess() are needed.

  7) Define internal functions for _Feature_ClearStructures(),
     _Feature_CreateStructures() and _Feature_InitStructures();

     a) The _Feature_ClearStructures() should simply null out pointers
        to extra structures on its first invocation, but thereafter it
        should free them and then null them.  Since the null-only step
        is done only once in gp_AttachFeature(), it seems reasonable to
        not bother with a more complicated _Feature_ClearStructures().
        But, as an extension is developed, the data structures change,
        so it is best to keep all this logic in one place.

     b) The _Feature_CreateStructures() should just allocate memory for
        but not initialize any vertex level and edge level data structures.
        Data structures maintained at the graph level, such as a stack or a
        list collection, should be created _and_ initialized.

     c) The _Feature_InitStructures() should invoke just the functions
        needed to initialize the custom VertexRec, VertexInfo and EdgeRec
        data members, if any.

  8) Define a function gp_DetachFeature() that invokes gp_RemoveExtension()
     This should be done for consistency, so that users of a feature
     do not attach it with gp_AttachFeature() and remove it with
     gp_RemoveExtension().  However, it may sometimes be necessary to
     run more code than just gp_RemoveExtension() when detaching a feature,
     e.g. some final result values of a feature may be saved to data
     available in the core graph or in other features.
 ********************************************************************/

/********************************************************************
 gp_AddExtension()

 @param theGraph - pointer to the graph to which the extension is being added
 @param pModuleID - address of the variable that contains the feature's
				 extension identifier.  If the variable is equal to zero,
				 it is assigned a positive number.  Thereafter, the variable
				 value can be used to find and remove the extension from any graph
 @param context - the data storage for the extension being added
               The context is owned by the extension and freed with freeContext()
 @param dupContext - a function capable of duplicating the context data
 @param freeContext - a function capable of freeing the context data
 @param functions - pointer to a table of functions stored in the data context.
                        The table of functions is an input and output parameter.
                        On input, the table consists of new function pointers
                        for functions being overloaded.
                        Any function not being overloaded must be NULL.
                        The non-NULL function pointers are used to overload
                        the functions in the graph, and the prior pointer values
                        in the graph are stored in the function table as output.
                        The context data therefore has the pointer to the base
                        function corresponding to any function its extension
                        module overloaded.

 The new extension is created and added to the graph.
 ********************************************************************/

int gp_AddExtension(graphP theGraph,
                    int  *pModuleID,
                    void *context,
                    void *(*dupContext)(void *, void *),
                    void (*freeContext)(void *),
                    graphFunctionTableP functions)
{
    graphExtensionP newExtension = NULL;

    if (theGraph == NULL || pModuleID == NULL ||
        context == NULL || dupContext == NULL || freeContext == NULL ||
        functions == NULL)
    {
        return NOTOK;
    }

    // If the extension already exists, then don't redefine it.
    if (gp_FindExtension(theGraph, *pModuleID, NULL) == TRUE)
    {
        return NOTOK;
    }

    // Assign a unique ID to the extension if it does not already have one
    if (*pModuleID == 0)
    {
    	*pModuleID = ++moduleIDGenerator;
    }

    // Allocate the new extension
    if ((newExtension = (graphExtensionP) malloc(sizeof(graphExtension))) == NULL)
    {
        return NOTOK;
    }

    // Assign the data payload of the extension
    newExtension->moduleID = *pModuleID;
    newExtension->context = context;
    newExtension->dupContext = dupContext;
    newExtension->freeContext = freeContext;
    newExtension->functions = functions;

    _OverloadFunctions(theGraph, functions);

    // Make the new linkages
    newExtension->next = (struct graphExtension *) theGraph->extensions;
    theGraph->extensions = newExtension;

    // The new extension was successfully added
    return OK;

}

/********************************************************************
 _OverloadFunctions()
 For each non-NULL function pointer, the pointer becomes the new value
 for the function in the graph, and the old function pointer in the graph
 is placed in the overload table.

 This way, when an extension function is invoked, it can choose to invoke
 the base function before or after whatever extension behavior it provides.

 Also, when it comes time to remove an extension, this extension system
 has access to the overload tables of all extensions so that it can unhook
 the functions of the module being removed from the chains of calls for
 each overloaded function.  This will involve some pointer changes in
 the overload tables of extensions other than the one being removed.
 ********************************************************************/

void _OverloadFunctions(graphP theGraph, graphFunctionTableP functions)
{
void **graphFunctionTable = (void **) &theGraph->functions;
void **newFunctionTable = (void **) functions;
int  numFunctions = sizeof(theGraph->functions) / sizeof(void *);
int  K;

    for (K = 0; K < numFunctions; K++)
    {
        if (newFunctionTable[K] != NULL)
        {
            void *fp = graphFunctionTable[K];
            graphFunctionTable[K] = newFunctionTable[K];
            newFunctionTable[K] = fp;
        }
    }
}

/********************************************************************
 gp_FindExtension()

 @param theGraph - the graph whose extension list is to be searched
 @param moduleID - the identifier of the module whose extension context is desired
 @param pContext - the return parameter that receives the value of the
                extension, if found.  This may be NULL if the extension was
                not found or if the extension context value was NULL.
 @return TRUE if the extension was found, NOTOK if not found
         If FALSE is returned, then the context returned is guaranteed to be NULL
         If TRUE is returned, the context returned may be NULL if that is the
         current value of the module extension
 ********************************************************************/

int gp_FindExtension(graphP theGraph, int moduleID, void **pContext)
{
    graphExtensionP first = NULL, next = NULL;

    if (pContext != NULL)
    {
        *pContext = NULL;
    }

    if (theGraph==NULL || moduleID==0)
    {
        return FALSE;
    }

    first = theGraph->extensions;

    while (first != NULL)
    {
        next = (graphExtensionP) first->next;
        if (first->moduleID == moduleID)
        {
            if (pContext != NULL)
            {
                *pContext = first->context;
            }
            return TRUE;
        }
        first = next;
    }

    return FALSE;
}

/********************************************************************
 gp_GetExtension()

 Calling this function is equivalent to invoking gp_FindExtension()
 except that some debuggers have difficulty stepping into a function
 that (properly) start by setting a local variable pointer to NULL
 when the debugger has watch expressions that dereference a pointer
 of the same name.  In such cases,

 MyContext *context = NULL;
 gp_FindExtension(theGraph, MYEXTENSION_ID, &context);

 can be replaced by

 MyContext *context = gp_GetExtension(theGraph, MYEXTENSION_ID);

 @param theGraph - the graph whose extension list is to be searched
 @param moduleID - the identifier of the module whose extension context is desired
 @return void pointer to the extension if found, or NULL if not found.
 ********************************************************************/
void *gp_GetExtension(graphP theGraph, int moduleID)
{
	void *context = NULL;
	int result = gp_FindExtension(theGraph, moduleID, &context);
	return result ? context : NULL;
}

/********************************************************************
 gp_RemoveExtension()
 @param theGraph - the graph from which to remove an extension
 @param moduleID - the ID of the module whose extension context is to be removed
 @return OK if the module is successfully removed or not in the list
         NOTOK for internal errors, such as invalid parameters
 ********************************************************************/
int gp_RemoveExtension(graphP theGraph, int moduleID)
{
    graphExtensionP prev, curr, next;

    if (theGraph==NULL || moduleID==0)
        return NOTOK;

    prev = NULL;
    curr = theGraph->extensions;
	next = NULL;

    while (curr != NULL)
    {
        next = (graphExtensionP) curr->next;

        if (curr->moduleID == moduleID)
            break;

        prev = curr;
        curr = next;
    }

    // An extension can only be removed if it is found.  Otherwise,
    // we return OK because the extension degenerately removed
    // (since it is already gone)
    if (curr != NULL)
    {
        _FixupFunctionTables(theGraph, curr);

        // Unhook the curr extension
        if (prev != NULL)
             prev->next = (struct graphExtension *) next;
        else theGraph->extensions = next;

        // Free the curr extension
        _FreeExtension(curr);
    }

    return OK;
}


/********************************************************************
 _FixupFunctionTables()

 Removes the functions in the curr function table from the function
 call lists established by the function tables of all extensions and
 theGraph.

 Since new extensions are prepended, extensions before curr may
 have further overloaded the functions in the curr function table.

 For a non-NULL function pointer in the curr table, if there is
 a preceding extension with the same function pointer non-NULL, then
 the function table of the closest such preceding extension points
 to the original overload function of the curr extension, and the
 curr extension contains the pointer to the base function behavior,
 so now the function table of that preceding extension must be changed
 to the function pointer value in the curr extension.
 ********************************************************************/

void _FixupFunctionTables(graphP theGraph, graphExtensionP curr)
{
    void **currFunctionTable = (void **) (curr->functions);
    int  numFunctions = sizeof(*(curr->functions)) / sizeof(void *);
    int  K;

    for (K = 0; K < numFunctions; K++)
    {
        if (currFunctionTable[K] != NULL)
        {
            void **nearestOverloadFunctionTable = (void **) &theGraph->functions;
            graphExtensionP pred = _FindNearestOverload(theGraph, curr, K);

            if (pred != NULL)
                nearestOverloadFunctionTable = (void **) pred->functions;

            nearestOverloadFunctionTable[K] = currFunctionTable[K];
        }
    }
}

/********************************************************************
 _FindNearestOverload()
 ********************************************************************/

graphExtensionP _FindNearestOverload(graphP theGraph, graphExtensionP target, int functionIndex)
{
    graphExtensionP curr = theGraph->extensions;
    graphExtensionP found = NULL;
    void **functionTable;

    while (curr != target)
    {
        functionTable = (void **) curr->functions;
        if (functionTable[functionIndex] != NULL)
            found = curr;

        curr = (graphExtensionP) curr->next;
    }

    return found;
}

/********************************************************************
 gp_CopyExtensions()
 ********************************************************************/

int gp_CopyExtensions(graphP dstGraph, graphP srcGraph)
{
    graphExtensionP next = NULL, newNext = NULL, newLast = NULL;

    if (srcGraph == NULL || dstGraph == NULL)
        return NOTOK;

    gp_FreeExtensions(dstGraph);

    next = srcGraph->extensions;

    while (next != NULL)
    {
        if ((newNext = (graphExtensionP) malloc(sizeof(graphExtension))) == NULL)
        {
            gp_FreeExtensions(dstGraph);
            return NOTOK;
        }

        newNext->moduleID = next->moduleID;
        newNext->context = next->dupContext(next->context, dstGraph);
        newNext->dupContext = next->dupContext;
        newNext->freeContext = next->freeContext;
        newNext->functions = next->functions;
        newNext->next = NULL;

        if (newLast != NULL)
            newLast->next = (struct graphExtension *) newNext;
        else
            dstGraph->extensions = newNext;

        newLast = newNext;
        next = (graphExtensionP) next->next;
    }

    return OK;
}

/********************************************************************
 gp_FreeExtensions()

 @param pFirst - pointer to head pointer of graph extension list

 Each graph extension is freed, including invoking the freeContext
 function provided when the extension was added.
 ********************************************************************/

void gp_FreeExtensions(graphP theGraph)
{
    if (theGraph != NULL)
    {
        graphExtensionP curr = theGraph->extensions;
        graphExtensionP next = NULL;

        while (curr != NULL)
        {
            next = (graphExtensionP) curr->next;
            _FreeExtension(curr);
            curr = next;
        }

        theGraph->extensions = NULL;
        _InitFunctionTable(theGraph);
    }
}

/********************************************************************
 _FreeExtension()
 ********************************************************************/
void _FreeExtension(graphExtensionP extension)
{
    if (extension->context != NULL && extension->freeContext != NULL)
    {
        extension->freeContext(extension->context);
    }
    free(extension);
}