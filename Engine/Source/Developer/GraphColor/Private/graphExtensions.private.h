#pragma once

/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#include "graphFunctionTable.h"

struct graphExtension
{
    int  moduleID;
    void *context;
    void *(*dupContext)(void *, void *);
    void (*freeContext)(void *);

    graphFunctionTableP functions;

    graphExtension *next;
};

typedef graphExtension * graphExtensionP;