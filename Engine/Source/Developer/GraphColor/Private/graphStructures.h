#pragma once

/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#include "appconst.h"
#include "listcoll.h"
#include "stack.h"

#include "graphFunctionTable.h"
#include "graphExtensions.private.h"

// A return value to indicate success prior to completely processing a graph, whereas
// OK signifies EMBEDDABLE (no unreducible obstructions) and NOTOK signifies an exception.
#define NONEMBEDDABLE   -1

// The initial setting for the edge storage capacity expressed as a constant factor of N,
// which is the number of vertices in the graph. By default, array E is allocated enough
// space to contain 3N edges, which is 6N arcs (half edges), but this initial setting
// can be overridden using gp_EnsureArcCapacity(), which is especially efficient if done
// before calling gp_InitGraph() or gp_Read().
#define DEFAULT_EDGE_LIMIT      3

/********************************************************************
 Edge Record Definition

 An edge is defined by a pair of edge records, or arcs, allocated in
 array E of a graph.  An edge record represents the edge in the
 adjacency list of each vertex to which the edge is incident.

 link[2]: the next and previous edge records (arcs) in the adjacency
          list that contains this edge record.

 v: The vertex neighbor of the vertex whose adjacency list contains
    this edge record (an index into array V).
 ********************************************************************/

typedef struct
{
	int  link[2];
	int  neighbor;
} edgeRec;

typedef edgeRec * edgeRecP;

#if NIL == 0
#define gp_IsArc(e) (e)
#define gp_IsNotArc(e) (!(e))
#define gp_GetFirstEdge(theGraph) (2)
#elif NIL == -1
#define gp_IsArc(e) ((e) != NIL)
#define gp_IsNotArc(e) ((e) == NIL)
#define gp_GetFirstEdge(theGraph) (0)
#else
#error NIL must be 0 or -1
#endif

#define gp_EdgeInUse(theGraph, e) (gp_IsVertex(gp_GetNeighbor(theGraph, e)))
#define gp_EdgeNotInUse(theGraph, e) (gp_IsNotVertex(gp_GetNeighbor(theGraph, e)))
#define gp_EdgeIndexBound(theGraph) (gp_GetFirstEdge(theGraph) + (theGraph)->arcCapacity)
#define gp_EdgeInUseIndexBound(theGraph) (gp_GetFirstEdge(theGraph) + (((theGraph)->M + sp_GetCurrentSize((theGraph)->edgeHoles)) << 1))

// An edge is represented by two consecutive edge records (arcs) in the edge array E.
// If an even number, xor 1 will add one; if an odd number, xor 1 will subtract 1
#define gp_GetTwinArc(theGraph, Arc) ((Arc) ^ 1)

// Access to adjacency list pointers
#define gp_GetNextArc(theGraph, e) (theGraph->E[e].link[0])
#define gp_GetPrevArc(theGraph, e) (theGraph->E[e].link[1])
#define gp_GetAdjacentArc(theGraph, e, theLink) (theGraph->E[e].link[theLink])

#define gp_SetNextArc(theGraph, e, newNextArc) (theGraph->E[e].link[0] = newNextArc)
#define gp_SetPrevArc(theGraph, e, newPrevArc) (theGraph->E[e].link[1] = newPrevArc)
#define gp_SetAdjacentArc(theGraph, e, theLink, newArc) (theGraph->E[e].link[theLink] = newArc)

// Access to vertex 'neighbor' member indicated by arc
#define gp_GetNeighbor(theGraph, e) (theGraph->E[e].neighbor)
#define gp_SetNeighbor(theGraph, e, v) (theGraph->E[e].neighbor = v)

#define gp_CopyEdgeRec(dstGraph, edst, srcGraph, esrc) (dstGraph->E[edst] = srcGraph->E[esrc])

/********************************************************************
 Vertex Record Definition

 This record definition provides the data members needed for the
 core structural information for both vertices and virtual vertices.
 Vertices are also equipped with additional information provided by
 the vertexInfo structure.

 The vertices of a graph are stored in the first N locations of array V.
 Virtual vertices are secondary vertices used to help represent the
 main vertices in substructural components of a graph (e.g. biconnected
 components).

 link[2]: the first and last edge records (arcs) in the adjacency list
          of the vertex.

 index: In vertices, stores either the depth first index of a vertex or
        the original array index of the vertex if the vertices of the
        graph are sorted by DFI.
        In virtual vertices, the index may be used to indicate the vertex
        that the virtual vertex represents, unless an algorithm has some
        other way of making the association (for example, the planarity
        algorithms rely on biconnected components and therefore place
        virtual vertices of a vertex at positions corresponding to the
        DFS children of the vertex).

 flags: Bits 0-15 reserved for library; bits 16 and higher for apps
        Bit 0: visited, for vertices and virtual vertices
				Use in lieu of TYPE_VERTEX_VISITED in K4 algorithm
		Bit 1: Obstruction type VERTEX_TYPE_SET (versus not set, i.e. VERTEX_TYPE_UNKNOWN)
		Bit 2: Obstruction type qualifier RYW (set) versus RXW (clear)
		Bit 3: Obstruction type qualifier high (set) versus low (clear)
 ********************************************************************/

typedef struct
{
	int  link[2];
	int  index;
	unsigned flags;
} vertexRec;

typedef vertexRec * vertexRecP;

// Accessors for vertex adjacency list links
#define gp_GetFirstArc(theGraph, v) (theGraph->V[v].link[0])
#define gp_GetLastArc(theGraph, v) (theGraph->V[v].link[1])
#define gp_GetArc(theGraph, v, theLink) (theGraph->V[v].link[theLink])

#define gp_SetFirstArc(theGraph, v, newFirstArc) (theGraph->V[v].link[0] = newFirstArc)
#define gp_SetLastArc(theGraph, v, newLastArc) (theGraph->V[v].link[1] = newLastArc)
#define gp_SetArc(theGraph, v, theLink, newArc) (theGraph->V[v].link[theLink] = newArc)

// Vertex conversions and iteration
#if NIL == 0
#define gp_IsVertex(v) (v)
#define gp_IsNotVertex(v) (!(v))

#define gp_GetFirstVertex(theGraph) (1)
#define gp_GetLastVertex(theGraph) ((theGraph)->N)
#define gp_VertexInRange(theGraph, v) ((v) <= (theGraph)->N)
#define gp_VertexInRangeDescending(theGraph, v) (v)

#define gp_PrimaryVertexIndexBound(theGraph) (gp_GetFirstVertex(theGraph) + (theGraph)->N)
#define gp_VertexIndexBound(theGraph) (gp_PrimaryVertexIndexBound(theGraph) + (theGraph)->N)

#define gp_GetFirstVirtualVertex(theGraph) (theGraph->N + 1)
#define gp_VirtualVertexInRange(theGraph, v) ((v) <= theGraph->N + theGraph->NV)

#elif NIL == -1
#define gp_IsVertex(v) ((v) != NIL)
#define gp_IsNotVertex(v) ((v) == NIL)

#define gp_GetFirstVertex(theGraph) (0)
#define gp_GetLastVertex(theGraph) ((theGraph)->N - 1)
#define gp_VertexInRange(theGraph, v) ((v) < (theGraph)->N)
#define gp_VertexInRangeDescending(theGraph, v) ((v) >= 0)

#define gp_PrimaryVertexIndexBound(theGraph) (gp_GetFirstVertex(theGraph) + (theGraph)->N)
#define gp_VertexIndexBound(theGraph) (gp_PrimaryVertexIndexBound(theGraph) + (theGraph)->N)

#define gp_GetFirstVirtualVertex(theGraph) (theGraph->N)
#define gp_VirtualVertexInRange(theGraph, v) ((v) < theGraph->N + theGraph->NV)

#else
#error NIL must be 0 or -1
#endif

// Accessors for vertex index
#define gp_GetVertexIndex(theGraph, v) (theGraph->V[v].index)
#define gp_SetVertexIndex(theGraph, v, theIndex) (theGraph->V[v].index = theIndex)

// Initializer for vertex flags
#define gp_InitVertexFlags(theGraph, v) (theGraph->V[v].flags = 0)

// Definitions and accessors for vertex flags
#define VERTEX_VISITED_MASK		1
#define gp_GetVertexVisited(theGraph, v) (theGraph->V[v].flags&VERTEX_VISITED_MASK)
#define gp_ClearVertexVisited(theGraph, v) (theGraph->V[v].flags &= ~VERTEX_VISITED_MASK)
#define gp_SetVertexVisited(theGraph, v) (theGraph->V[v].flags |= VERTEX_VISITED_MASK)

#define gp_CopyVertexRec(dstGraph, vdst, srcGraph, vsrc) (dstGraph->V[vdst] = srcGraph->V[vsrc])
#define gp_SwapVertexRec(dstGraph, vdst, srcGraph, vsrc) \
	{ \
		vertexRec tempV = dstGraph->V[vdst]; \
		dstGraph->V[vdst] = srcGraph->V[vsrc]; \
		srcGraph->V[vsrc] = tempV; \
	}

/********************************************************************
 Graph structure definition
        V : Array of vertex records (allocated size N + NV)
        VI: Array of additional vertexInfo structures (allocated size N)
        N : Number of primary vertices (the "order" of the graph)
        NV: Number of virtual vertices (currently always equal to N)

        E : Array of edge records (edge records come in pairs and represent half edges, or arcs)
        M: Number of edges (the "size" of the graph)
        arcCapacity: the maximum number of edge records allowed in E (the size of E)
        edgeHoles: free locations in E where edges have been deleted

        theStack: Used by various graph routines needing a stack
        internalFlags: Additional state information about the graph

        extensions: a list of extension data structures
        functions: a table of function pointers that can be overloaded to provide
                   extension behaviors to the graph
*/

struct baseGraphStructure
{
        vertexRecP V;

        int N, NV;

        edgeRecP E;
        int M, arcCapacity;
        stackP edgeHoles;

        stackP theStack;

        graphExtensionP extensions;
        graphFunctionTable functions;
};

typedef baseGraphStructure * graphP;

// Methods for attaching an arc into the adjacency list or detaching an arc from it.
// The terms AddArc, InsertArc and DeleteArc are not used because the arcs are not
// inserted or added to or deleted from storage (only whole edges are inserted or deleted)
void	gp_AttachArc(graphP theGraph, int v, int e, int link, int newArc);
void 	gp_DetachArc(graphP theGraph, int arc);