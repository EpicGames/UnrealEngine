/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#include "CoreMinimal.h"

#include "graphStructures.h"
#include "graph.h"

/* Internal util functions for FUNCTION POINTERS */

int  _HideVertex(graphP theGraph, int vertex);
void _HideEdge(graphP theGraph, int arcPos);
void _RestoreEdge(graphP theGraph, int arcPos);
int  _IdentifyVertices(graphP theGraph, int u, int v, int eBefore);
int  _RestoreVertex(graphP theGraph);

/********************************************************************
 Private functions, except exported within library
 ********************************************************************/

void _ClearVisitedFlags(graphP theGraph);
void _ClearVertexVisitedFlags(graphP theGraph, int);

int  _HideInternalEdges(graphP theGraph, int vertex);
int  _RestoreInternalEdges(graphP theGraph, int stackBottom);
int  _RestoreHiddenEdges(graphP theGraph, int stackBottom);

void _InitFunctionTable(graphP theGraph);

/********************************************************************
 Private functions.
 ********************************************************************/

void _InitVertices(graphP theGraph);
void _InitEdges(graphP theGraph);

void _ClearGraph(graphP theGraph);

/* Private functions for which there are FUNCTION POINTERS */

int  _InitGraph(graphP theGraph, int N);
void _ReinitializeGraph(graphP theGraph);
int  _EnsureArcCapacity(graphP theGraph, int requiredArcCapacity);

/********************************************************************
 gp_New()
 Constructor for graph object.
 Can create two graphs if restricted to no dynamic memory.
 ********************************************************************/

graphP gp_New()
{
graphP theGraph = (graphP) malloc(sizeof(baseGraphStructure));

     if (theGraph != NULL)
     {
         theGraph->E = NULL;
         theGraph->V = NULL;

         theGraph->theStack = NULL;

         theGraph->edgeHoles = NULL;

         theGraph->extensions = NULL;

         _InitFunctionTable(theGraph);

         _ClearGraph(theGraph);
     }

     return theGraph;
}

/********************************************************************
 _InitFunctionTable()

 If you add functions to the function table, then they must be
 initialized here, but you must also add the new function pointer
 to the definition of the graphFunctionTable in graphFunctionTable.h

 Function headers for the functions used to initialize the table are
 classified at the top of this file as either imported from other
 compilation units (extern) or private to this compilation unit.
 Search for FUNCTION POINTERS in this file to see where to add the
 function header.
 ********************************************************************/

void _InitFunctionTable(graphP theGraph)
{
	theGraph->functions.fpInitGraph = _InitGraph;

	theGraph->functions.fpHideEdge = _HideEdge;
	theGraph->functions.fpRestoreVertex = _RestoreVertex;
	theGraph->functions.fpIdentifyVertices = _IdentifyVertices;
}

/********************************************************************
 gp_InitGraph()
 Allocates memory for vertex and edge records now that N is known.
 The arcCapacity is set to (2 * DEFAULT_EDGE_LIMIT * N) unless it
	 has already been set by gp_EnsureArcCapacity()

 For V, we need 2N vertex records, N for vertices and N for virtual vertices (root copies).

 For VI, we need N vertexInfo records.

 For E, we need arcCapacity edge records.

 The stack, initially empty, is made big enough for a pair of integers
	 per edge record (2 * arcCapacity), or 6N integers if the arcCapacity
	 was set below the default value.

 The edgeHoles stack, initially empty, is set to arcCapacity / 2,
	 which is big enough to push every edge (to indicate an edge
	 you only need to indicate one of its two edge records)

  Returns OK on success, NOTOK on all failures.
          On NOTOK, graph extensions are freed so that the graph is
          returned to the post-condition of gp_New().
 ********************************************************************/

int gp_InitGraph(graphP theGraph, int N)
{
	// valid params check
	if (theGraph == NULL || N <= 0)
        return NOTOK;

	// Should not call init a second time; use reinit
	if (theGraph->N)
		return NOTOK;

    return theGraph->functions.fpInitGraph(theGraph, N);
}

int  _InitGraph(graphP theGraph, int N)
{
	int  Vsize, VIsize, Esize, stackSize;

	 // Compute the vertex and edge capacities of the graph
     theGraph->N = N;
     theGraph->NV = N;
     theGraph->arcCapacity = theGraph->arcCapacity > 0 ? theGraph->arcCapacity : 2*DEFAULT_EDGE_LIMIT*N;
	 VIsize = gp_PrimaryVertexIndexBound(theGraph);
     Vsize = gp_VertexIndexBound(theGraph);
     Esize = gp_EdgeIndexBound(theGraph);

     // Stack size is 2 integers per arc, or 6 integers per vertex in case of small arcCapacity
     stackSize = 2 * Esize;
     stackSize = stackSize < 6*N ? 6*N : stackSize;

     // Allocate memory as described above
     if ((theGraph->V = (vertexRecP) calloc(Vsize, sizeof(vertexRec))) == NULL ||
    	 (theGraph->E = (edgeRecP) calloc(Esize, sizeof(edgeRec))) == NULL ||
         (theGraph->theStack = sp_New(stackSize)) == NULL ||
         (theGraph->edgeHoles = sp_New(Esize / 2)) == NULL ||
         0)
     {
         _ClearGraph(theGraph);
         return NOTOK;
     }

     // Initialize memory
     _InitVertices(theGraph);
     _InitEdges(theGraph);

     return OK;
}

/********************************************************************
 _InitVertices()
 ********************************************************************/
void _InitVertices(graphP theGraph)
{
#if NIL == 0
	memset(theGraph->V, NIL_CHAR, gp_VertexIndexBound(theGraph) * sizeof(vertexRec));
#elif NIL == -1
	int v;

	memset(theGraph->V, NIL_CHAR, gp_VertexIndexBound(theGraph) * sizeof(vertexRec));

	for (v = gp_GetFirstVertex(theGraph); gp_VertexInRange(theGraph, v); v++)
	    gp_InitVertexFlags(theGraph, v);
#else
#error NIL must be 0 or -1
#endif
}

/********************************************************************
 _InitEdges()
 ********************************************************************/
void _InitEdges(graphP theGraph)
{
#if NIL == 0
	memset(theGraph->E, NIL_CHAR, gp_EdgeIndexBound(theGraph) * sizeof(edgeRec));
#elif NIL == -1
	int e, Esize;

	memset(theGraph->E, NIL_CHAR, gp_EdgeIndexBound(theGraph) * sizeof(edgeRec));

	Esize = gp_EdgeIndexBound(theGraph);
    for (e = gp_GetFirstEdge(theGraph); e < Esize; e++)
        gp_InitEdgeFlags(theGraph, e);
#else
#error NIL must be 0 or -1
#endif
}

/********************************************************************
 gp_GetArcCapacity()
 Returns the arcCapacity of theGraph, which is twice the maximum
 number of edges that can be added to the theGraph.
 ********************************************************************/
int gp_GetArcCapacity(graphP theGraph)
{
	return theGraph->arcCapacity - gp_GetFirstEdge(theGraph);
}

/********************************************************************
 gp_EnsureArcCapacity()
 This method ensures that theGraph is or will be capable of storing
 at least requiredArcCapacity edge records.  Two edge records are
 needed per edge.

 This method is most performant when invoked immediately after
 gp_New(), since it must only set the arcCapacity and then let
 normal initialization to occur through gp_InitGraph().

 This method is also a constant time operation if the graph already
 has at least the requiredArcCapacity, since it will return OK
 without making any structural changes.

 This method is generally more performant if it is invoked before
 attaching extensions to the graph.  Some extensions associate
 parallel data with edge records, which is a faster operation if
 the associated data is created and initialized only after the
 proper arcCapacity is specified.

 If the graph has been initialized and has a lower arc capacity,
 then the array of edge records is reallocated to satisfy the
 requiredArcCapacity.  The new array contains the old edges and
 edge holes at the same locations, and all newly created edge records
 are initialized.

 Also, if the arc capacity must be increased, then the
 arcCapacity member of theGraph is changed and both
 theStack and edgeHoles are expanded (since the sizes of both
 are based on the arc capacity).

 Extensions that add to data associated with edges must overload
 this method to ensure capacity in the parallel extension data
 structures.  An extension can return NOTOK if it does not
 support arc capacity expansion.  The extension function will
 not be called if arcCapacity is expanded before the graph is
 initialized, and it is assumed that extensions will allocate
 parallel data structures according to the arc capacity.

 If an extension supports arc capacity expansion, then higher
 performance can be obtained by using the method of unhooking
 the initializers for individual edge records before invoking
 the superclass version of fpEnsureArcCapacity().  Ideally,
 application authors should ensure the proper arc capacity before
 attaching extensions to achieve better performance.

 Returns NOTOK on failure to reallocate the edge record array to
         satisfy the requiredArcCapacity, or if the requested
         capacity is odd
         OK if reallocation is not required or if reallocation succeeds
 ********************************************************************/
int gp_EnsureArcCapacity(graphP theGraph, int requiredArcCapacity)
{
	if (theGraph == NULL || requiredArcCapacity <= 0)
		return NOTOK;

	// Train callers to only ask for an even number of arcs, since
	// two are required per edge or directed edge.
	if (requiredArcCapacity & 1)
		return NOTOK;

    if (theGraph->arcCapacity >= requiredArcCapacity)
    	return OK;

    // In the special case where gp_InitGraph() has not yet been called,
    // we can simply set the higher arcCapacity since normal initialization
    // will then allocate the correct number of edge records.
    if (theGraph->N == 0)
    {
    	theGraph->arcCapacity = requiredArcCapacity;
    	return OK;
    }

    return NOTOK;
}

/********************************************************************
 _ClearVisitedFlags()
 ********************************************************************/

void _ClearVisitedFlags(graphP theGraph)
{
	_ClearVertexVisitedFlags(theGraph, TRUE);
}

/********************************************************************
 _ClearVertexVisitedFlags()
 ********************************************************************/

void _ClearVertexVisitedFlags(graphP theGraph, int includeVirtualVertices)
{
	int  v;

	for (v = gp_GetFirstVertex(theGraph); gp_VertexInRange(theGraph, v); v++)
        gp_ClearVertexVisited(theGraph, v);

	if (includeVirtualVertices)
		for (v = gp_GetFirstVirtualVertex(theGraph); gp_VirtualVertexInRange(theGraph, v); v++)
	        gp_ClearVertexVisited(theGraph, v);
}

/********************************************************************
 _ClearGraph()
 Clears all memory used by the graph, restoring it to the state it
 was in immediately after gp_New() created it.
 ********************************************************************/

void _ClearGraph(graphP theGraph)
{
	if (theGraph->V != NULL)
     {
          free(theGraph->V);
          theGraph->V = NULL;
     }
     if (theGraph->E != NULL)
     {
          free(theGraph->E);
          theGraph->E = NULL;
     }

     theGraph->N = 0;
     theGraph->NV = 0;
     theGraph->M = 0;
     theGraph->arcCapacity = 0;

     sp_Free(&theGraph->theStack);

     sp_Free(&theGraph->edgeHoles);

     gp_FreeExtensions(theGraph);
}

/********************************************************************
 gp_Free()
 Frees G and V, then the graph record.  Then sets your pointer to NULL
 (so you must pass the address of your pointer).
 ********************************************************************/

void gp_Free(graphP *pGraph)
{
     if (pGraph == NULL) return;
     if (*pGraph == NULL) return;

     _ClearGraph(*pGraph);

     free(*pGraph);
     *pGraph = NULL;
}

/********************************************************************
 gp_CopyGraph()
 Copies the content of the srcGraph into the dstGraph.  The dstGraph
 must have been previously initialized with the same number of
 vertices as the srcGraph (e.g. gp_InitGraph(dstGraph, srcGraph->N).

 Returns OK for success, NOTOK for failure.
 ********************************************************************/

int  gp_CopyGraph(graphP dstGraph, graphP srcGraph)
{
int  v, e, Esize;

     // Parameter checks
     if (dstGraph == NULL || srcGraph == NULL)
     {
         return NOTOK;
     }

     // The graphs need to be the same order and initialized
     if (dstGraph->N != srcGraph->N || dstGraph->N == 0)
     {
         return NOTOK;
     }

     // Ensure dstGraph has the required arc capacity; this expands
     // dstGraph if needed, but does not contract.  An error is only
     // returned if the expansion fails.
     if (gp_EnsureArcCapacity(dstGraph, srcGraph->arcCapacity) != OK)
     {
    	 return NOTOK;
     }

     // Copy the primary vertices.  Augmentations to vertices created
     // by extensions are copied below by gp_CopyExtensions()
     for (v = gp_GetFirstVertex(srcGraph); gp_VertexInRange(srcGraph, v); v++)
     {
    	 gp_CopyVertexRec(dstGraph, v, srcGraph, v);
     }

     // Copy the virtual vertices.  Augmentations to virtual vertices created
     // by extensions are copied below by gp_CopyExtensions()
     for (v = gp_GetFirstVirtualVertex(srcGraph); gp_VirtualVertexInRange(srcGraph, v); v++)
     {
    	 gp_CopyVertexRec(dstGraph, v, srcGraph, v);
     }

     // Copy the basic EdgeRec structures.  Augmentations to the edgeRec structure
     // created by extensions are copied below by gp_CopyExtensions()
     Esize = gp_EdgeIndexBound(srcGraph);
     for (e = gp_GetFirstEdge(theGraph); e < Esize; e++)
    	 gp_CopyEdgeRec(dstGraph, e, srcGraph, e);

     // Give the dstGraph the same size and intrinsic properties
     dstGraph->N = srcGraph->N;
     dstGraph->NV = srcGraph->NV;
     dstGraph->M = srcGraph->M;

     sp_Copy(dstGraph->theStack, srcGraph->theStack);
     sp_Copy(dstGraph->edgeHoles, srcGraph->edgeHoles);

     // Copy the set of extensions, which includes copying the
     // extension data as well as the function overload tables
     if (gp_CopyExtensions(dstGraph, srcGraph) != OK)
    	 return NOTOK;

     // Copy the graph's function table, which has the pointers to
     // the most recent extension overloads of each function (or
     // the original function pointer if a particular function has
     // not been overloaded).
     // This must be done after copying the extension because the
     // first step of copying the extensions is to delete the
     // dstGraph extensions, which clears its function table.
     // Therefore, no good to assign the srcGraph functions *before*
     // copying the extensions because the assignment would be wiped out
     // This, in turn, means that the DupContext function of an extension
     // *cannot* depend on any extension function overloads; the extension
     // must directly invoke extension functions only.
     dstGraph->functions = srcGraph->functions;

     return OK;
}

/********************************************************************
 gp_IsNeighbor()

 Checks whether v is already in u's adjacency list, i.e. does the arc
 u -> v exist.
 If there is an edge record for v in u's list, but it is marked INONLY,
 then it represents the arc v->u but not u->v, so it is ignored.

 Returns TRUE or FALSE.
 ********************************************************************/

int  gp_IsNeighbor(graphP theGraph, int u, int v)
{
	int  e = gp_GetFirstArc(theGraph, u);

     while (gp_IsArc(e))
     {
          if (gp_GetNeighbor(theGraph, e) == v)
          {
              //if (gp_GetDirection(theGraph, e) != EDGEFLAG_DIRECTION_INONLY)
            	  return TRUE;
          }
          e = gp_GetNextArc(theGraph, e);
     }
     return FALSE;
}

/********************************************************************
 gp_GetNeighborEdgeRecord()
 Searches the adjacency list of u to obtains the edge record for v.

 NOTE: The caller should check whether the edge record is INONLY;
       This method returns any edge record representing a connection
       between vertices u and v, so this method can return an
       edge record even if gp_IsNeighbor(theGraph, u, v) is false (0).
       To filter out INONLY edge records, use gp_GetDirection() on
       the edge record returned by this method.

 Returns NIL if there is no edge record indicating v in u's adjacency
         list, or the edge record location otherwise.
 ********************************************************************/

int  gp_GetNeighborEdgeRecord(graphP theGraph, int u, int v)
{
	int  e;

     if (gp_IsNotVertex(u) || gp_IsNotVertex(v))
    	 return NIL + NOTOK - NOTOK;

     e = gp_GetFirstArc(theGraph, u);
     while (gp_IsArc(e))
     {
          if (gp_GetNeighbor(theGraph, e) == v)
        	  return e;

          e = gp_GetNextArc(theGraph, e);
     }
     return NIL;
}

/********************************************************************
 gp_GetVertexDegree()

 Counts the number of edge records in the adjacency list of a given
 vertex V.

 Note: For digraphs, this method returns the total degree of the
       vertex, including outward arcs (undirected and OUTONLY)
       as well as INONLY arcs.  Other functions are defined to get
       the in-degree or out-degree of the vertex.

 Note: This function determines the degree by counting.  An extension
       could cache the degree value of each vertex and update the
       cached value as edges are added and deleted.
 ********************************************************************/

int  gp_GetVertexDegree(graphP theGraph, int v)
{
	int  e, degree;

     if (theGraph==NULL || gp_IsNotVertex(v))
    	 return 0 + NOTOK - NOTOK;

     degree = 0;

     e = gp_GetFirstArc(theGraph, v);
     while (gp_IsArc(e))
     {
         degree++;
         e = gp_GetNextArc(theGraph, e);
     }

     return degree;
}

/********************************************************************
 gp_AttachArc()

 This routine adds newArc into v's adjacency list at a position
 adjacent to the edge record for e, either before or after e,
 depending on link.  If e is not an arc (e.g. if e is NIL),
 then link is assumed to indicate whether the new arc is to be
 placed at the beginning or end of v's adjacency list.

 NOTE: The caller can pass NIL for v if e is not NIL, since the
       vertex is implied (gp_GetNeighbor(theGraph, eTwin))

 The arc is assumed to already exist in the data structure (i.e.
 the storage of edges), as only a whole edge (two arcs) can be
 inserted into or deleted from the data structure.  Hence there is
 no such thing as gp_InsertArc() or gp_DeleteArc().

 See also gp_DetachArc(), gp_InsertEdge() and gp_DeleteEdge()
 ********************************************************************/

void gp_AttachArc(graphP theGraph, int v, int e, int link, int newArc)
{
	if (gp_IsArc(e))
     {
    	 int e2 = gp_GetAdjacentArc(theGraph, e, link);

         // e's link is newArc, and newArc's 1^link is e
    	 gp_SetAdjacentArc(theGraph, e, link, newArc);
    	 gp_SetAdjacentArc(theGraph, newArc, 1^link, e);

    	 // newArcs's link is e2
    	 gp_SetAdjacentArc(theGraph, newArc, link, e2);

    	 // if e2 is an arc, then e2's 1^link is newArc, else v's 1^link is newArc
    	 if (gp_IsArc(e2))
    		 gp_SetAdjacentArc(theGraph, e2, 1^link, newArc);
    	 else
    		 gp_SetArc(theGraph, v, 1^link, newArc);
     }
     else
     {
    	 int e2 = gp_GetArc(theGraph, v, link);

    	 // v's link is newArc, and newArc's 1^link is NIL
    	 gp_SetArc(theGraph, v, link, newArc);
    	 gp_SetAdjacentArc(theGraph, newArc, 1^link, NIL);

    	 // newArcs's elink is e2
    	 gp_SetAdjacentArc(theGraph, newArc, link, e2);

    	 // if e2 is an arc, then e2's 1^link is newArc, else v's 1^link is newArc
    	 if (gp_IsArc(e2))
    		 gp_SetAdjacentArc(theGraph, e2, 1^link, newArc);
    	 else
    		 gp_SetArc(theGraph, v, 1^link, newArc);
     }
}

/****************************************************************************
 gp_DetachArc()

 This routine detaches arc from its adjacency list, but it does not delete
 it from the data structure (only a whole edge can be deleted).

 Some algorithms must temporarily detach an edge, perform some calculation,
 and eventually put the edge back. This routine supports that operation.
 The neighboring adjacency list nodes are cross-linked, but the two link
 members of the arc are retained, so the arc can be reattached later by
 invoking _RestoreArc().  A sequence of detached arcs can only be restored
 in the exact opposite order of their detachment.  Thus, algorithms do not
 directly use this method to implement the temporary detach/restore method.
 Instead, gp_HideEdge() and gp_RestoreEdge are used, and algorithms push
 edge hidden edge onto the stack.  One example of this stack usage is
 provided by detaching edges with gp_ContractEdge() or gp_IdentifyVertices(),
 and reattaching with gp_RestoreIdentifications(), which unwinds the stack
 by invoking gp_RestoreVertex().
 ****************************************************************************/

void gp_DetachArc(graphP theGraph, int arc)
{
	int nextArc = gp_GetNextArc(theGraph, arc),
	    prevArc = gp_GetPrevArc(theGraph, arc);

	    if (gp_IsArc(nextArc))
	    	gp_SetPrevArc(theGraph, nextArc, prevArc);
	    else
	    	gp_SetLastArc(theGraph, gp_GetNeighbor(theGraph, gp_GetTwinArc(theGraph, arc)), prevArc);

	    if (gp_IsArc(prevArc))
	    	gp_SetNextArc(theGraph, prevArc, nextArc);
	    else
	    	gp_SetFirstArc(theGraph, gp_GetNeighbor(theGraph, gp_GetTwinArc(theGraph, arc)), nextArc);
}

/********************************************************************
 gp_AddEdge()
 Adds the undirected edge (u,v) to the graph by placing edge records
 representing u into v's circular edge record list and v into u's
 circular edge record list.

 upos receives the location in G where the u record in v's list will be
        placed, and vpos is the location in G of the v record we placed in
 u's list.  These are used to initialize the short circuit links.

 ulink (0|1) indicates whether the edge record to v in u's list should
        become adjacent to u by its 0 or 1 link, i.e. u[ulink] == vpos.
 vlink (0|1) indicates whether the edge record to u in v's list should
        become adjacent to v by its 0 or 1 link, i.e. v[vlink] == upos.

 ********************************************************************/

int  gp_AddEdge(graphP theGraph, int u, int ulink, int v, int vlink)
{
	int  upos, vpos;

     if (theGraph==NULL || u < gp_GetFirstVertex(theGraph) || v < gp_GetFirstVertex(theGraph) ||
    		 !gp_VirtualVertexInRange(theGraph, u) || !gp_VirtualVertexInRange(theGraph, v))
         return NOTOK;

     /* We enforce the edge limit */

     if (theGraph->M >= theGraph->arcCapacity/2)
         return NONEMBEDDABLE;

     if (sp_NonEmpty(theGraph->edgeHoles))
     {
         sp_Pop(theGraph->edgeHoles, vpos);
     }
     else
         vpos = gp_EdgeInUseIndexBound(theGraph);

     upos = gp_GetTwinArc(theGraph, vpos);

     gp_SetNeighbor(theGraph, upos, v);
     gp_AttachArc(theGraph, u, NIL, ulink, upos);
     gp_SetNeighbor(theGraph, vpos, u);
     gp_AttachArc(theGraph, v, NIL, vlink, vpos);

     theGraph->M++;
     return OK;
}

/********************************************************************
 _RestoreArc()
 This routine reinserts an arc into the edge list from which it
 was previously removed by gp_DetachArc().

 The assumed processing model is that arcs will be restored in reverse
 of the order in which they were hidden, i.e. it is assumed that the
 hidden arcs will be pushed on a stack and the arcs will be popped
 from the stack for restoration.
 ********************************************************************/

void _RestoreArc(graphP theGraph, int arc)
{
	int nextArc = gp_GetNextArc(theGraph, arc),
	prevArc = gp_GetPrevArc(theGraph, arc);

	if (gp_IsArc(nextArc))
		gp_SetPrevArc(theGraph, nextArc, arc);
	else
		gp_SetLastArc(theGraph, gp_GetNeighbor(theGraph, gp_GetTwinArc(theGraph, arc)), arc);

    if (gp_IsArc(prevArc))
    	gp_SetNextArc(theGraph, prevArc, arc);
    else
    	gp_SetFirstArc(theGraph, gp_GetNeighbor(theGraph, gp_GetTwinArc(theGraph, arc)), arc);
}

/********************************************************************
 gp_HideEdge()
 This routine removes the two arcs of an edge from the adjacency lists
 of its endpoint vertices, but does not delete them from the storage
 data structure.

 Many algorithms must temporarily remove an edge, perform some
 calculation, and eventually put the edge back. This routine supports
 that operation.

 For each arc, the neighboring adjacency list nodes are cross-linked,
 but the links in the arc are retained because they indicate the
 neighbor arcs to which the arc can be reattached by gp_RestoreEdge().
 ********************************************************************/

void gp_HideEdge(graphP theGraph, int e)
{
	theGraph->functions.fpHideEdge(theGraph, e);
}

void _HideEdge(graphP theGraph, int e)
{
	gp_DetachArc(theGraph, e);
	gp_DetachArc(theGraph, gp_GetTwinArc(theGraph, e));
}

/********************************************************************
 gp_RestoreEdge()
 This routine reinserts two two arcs of an edge into the adjacency
 lists of the edge's endpoints, the arcs having been previously
 removed by gp_HideEdge().

 The assumed processing model is that edges will be restored in
 reverse of the order in which they were hidden, i.e. it is assumed
 that the hidden edges will be pushed on a stack and the edges will
 be popped from the stack for restoration.

 Note: Since both arcs of an edge are restored, only one arc need
        be pushed on the stack for restoration.  This routine
        restores the two arcs in the opposite order from the order
        in which they are hidden by gp_HideEdge().
 ********************************************************************/

void gp_RestoreEdge(graphP theGraph, int e)
{
	_RestoreArc(theGraph, gp_GetTwinArc(theGraph, e));
	_RestoreArc(theGraph, e);
}

/********************************************************************
 _RestoreHiddenEdges()

 Each entry on the stack, down to stackBottom, is assumed to be an
 edge record (arc) pushed in concert with invoking gp_HideEdge().
 Each edge is restored using gp_RestoreEdge() in exact reverse of the
 hiding order.  The stack is reduced in size to stackBottom.

 Returns OK on success, NOTOK on internal failure.
 ********************************************************************/

int  _RestoreHiddenEdges(graphP theGraph, int stackBottom)
{
	int  e;

	 while (sp_GetCurrentSize(theGraph->theStack) > stackBottom)
	 {
		  sp_Pop(theGraph->theStack, e);
		  if (gp_IsNotArc(e))
			  return NOTOK;
		  gp_RestoreEdge(theGraph, e);
	 }

	 return OK;
}

/********************************************************************
 gp_HideVertex()

 Pushes onto the graph's stack and hides all arc nodes of the vertex.
 Additional integers are then pushed so that the result is reversible
 by gp_RestoreVertex().  See that method for details on the expected
 stack segment.

 Returns OK for success, NOTOK for internal failure.
 ********************************************************************/

int  gp_HideVertex(graphP theGraph, int vertex)
{
	if (gp_IsNotVertex(vertex))
		return NOTOK;

	return _HideVertex(theGraph, vertex);
}

int  _HideVertex(graphP theGraph, int vertex)
{
	int hiddenEdgeStackBottom = sp_GetCurrentSize(theGraph->theStack);
	int e = gp_GetFirstArc(theGraph, vertex);

    // Cycle through all the edges, pushing and hiding each
    while (gp_IsArc(e))
    {
        sp_Push(theGraph->theStack, e);
        gp_HideEdge(theGraph, e);
        e = gp_GetNextArc(theGraph, e);
    }

    // Push the additional integers needed by gp_RestoreVertex()
	sp_Push(theGraph->theStack, hiddenEdgeStackBottom);
	sp_Push(theGraph->theStack, NIL);
	sp_Push(theGraph->theStack, NIL);
	sp_Push(theGraph->theStack, NIL);
	sp_Push(theGraph->theStack, NIL);
	sp_Push(theGraph->theStack, NIL);
	sp_Push(theGraph->theStack, vertex);

    return OK;
}

/********************************************************************
 gp_ContractEdge()

 Contracts the edge e=(u,v).  This hides the edge (both e and its
 twin arc), and it also identifies vertex v with u.
 See gp_IdentifyVertices() for further details.

 Returns OK for success, NOTOK for internal failure.
 ********************************************************************/

int gp_ContractEdge(graphP theGraph, int e)
{
	int eBefore, u, v;

	if (gp_IsNotArc(e))
		return NOTOK;

	u = gp_GetNeighbor(theGraph, gp_GetTwinArc(theGraph, e));
	v = gp_GetNeighbor(theGraph, e);

	eBefore = gp_GetNextArc(theGraph, e);
	sp_Push(theGraph->theStack, e);
	gp_HideEdge(theGraph, e);

	return gp_IdentifyVertices(theGraph, u, v, eBefore);
}

/********************************************************************
 gp_IdentifyVertices()

 Identifies vertex v with vertex u by transferring all adjacencies
 of v to u.  Any duplicate edges are removed as described below.
 The non-duplicate edges of v are added to the adjacency list of u
 without disturbing their relative order, and they are added before
 the edge record eBefore in u's list. If eBefore is NIL, then the
 edges are simply appended to u's list.

 If u and v are adjacent, then gp_HideEdge() is invoked to remove
 the edge e=(u,v). Then, the edges of v that indicate neighbors of
 u are also hidden.  This is done by setting the visited flags of
 u's neighbors, then traversing the adjacency list of v.  For each
 visited neighbor of v, the edge is hidden because it would duplicate
 an adjacency already expressed in u's list. Finally, the remaining
 edges of v are moved to u's list, and each twin arc is adjusted
 to indicate u as a neighbor rather than v.

 This routine assumes that the visited flags are clear beforehand,
 and visited flag settings made herein are cleared before returning.

 The following are pushed, in order, onto the graph's built-in stack:
 1) an integer for each hidden edge
 2) the stack size before any hidden edges were pushed
 3) six integers that indicate u, v and the edges moved from v to u

 An algorithm that identifies a series of vertices, either through
 directly calling this method or via gp_ContractEdge(), can unwind
 the identifications using gp_RestoreIdentifications(), which
 invokes gp_RestoreVertex() repeatedly.

 Returns OK on success, NOTOK on internal failure
 ********************************************************************/

int gp_IdentifyVertices(graphP theGraph, int u, int v, int eBefore)
{
	return theGraph->functions.fpIdentifyVertices(theGraph, u, v, eBefore);
}

int _IdentifyVertices(graphP theGraph, int u, int v, int eBefore)
{
	int e = gp_GetNeighborEdgeRecord(theGraph, u, v);
	int hiddenEdgeStackBottom, eBeforePred;

	// If the vertices are adjacent, then the identification is
	// essentially an edge contraction with a bit of fixup.
	if (gp_IsArc(e))
	{
		int result = gp_ContractEdge(theGraph, e);

		// The edge contraction operation pushes one hidden edge then
	    // recursively calls this method. This method then pushes K
	    // hidden edges then an integer indicating where the top of
	    // stack was before the edges were hidden. That integer
	    // indicator must be decremented, thereby incrementing the
		// number of hidden edges to K+1.
		// After pushing the K hidden edges and the stackBottom of
		// the hidden edges, the recursive call to this method pushes
		// six more integers to indicate edges that were moved from
		// v to u, so the "hidden edges stackBottom" is in the next
		// position down.
		int hiddenEdgesStackBottomIndex = sp_GetCurrentSize(theGraph->theStack)-7;
		int hiddenEdgesStackBottomValue = sp_Get(theGraph->theStack, hiddenEdgesStackBottomIndex);

		sp_Set(theGraph->theStack, hiddenEdgesStackBottomIndex,	hiddenEdgesStackBottomValue - 1);

		return result;
	}

	// Now, u and v are not adjacent. Before we do any edge hiding or
	// moving, we record the current stack size, as this is the
	// stackBottom for the edges that will be hidden next.
	hiddenEdgeStackBottom = sp_GetCurrentSize(theGraph->theStack);

	// Mark as visited all neighbors of u
    e = gp_GetFirstArc(theGraph, u);
    while (gp_IsArc(e))
    {
    	 if (gp_GetVertexVisited(theGraph, gp_GetNeighbor(theGraph, e)))
    		 return NOTOK;

         gp_SetVertexVisited(theGraph, gp_GetNeighbor(theGraph, e));
         e = gp_GetNextArc(theGraph, e);
    }

	// For each edge record of v, if the neighbor is visited, then
	// push and hide the edge.
    e = gp_GetFirstArc(theGraph, v);
    while (gp_IsArc(e))
    {
         if (gp_GetVertexVisited(theGraph, gp_GetNeighbor(theGraph, e)))
         {
             sp_Push(theGraph->theStack, e);
             gp_HideEdge(theGraph, e);
         }
         e = gp_GetNextArc(theGraph, e);
    }

	// Mark as unvisited all neighbors of u
    e = gp_GetFirstArc(theGraph, u);
    while (gp_IsArc(e))
    {
    	 gp_ClearVertexVisited(theGraph, gp_GetNeighbor(theGraph, e));
         e = gp_GetNextArc(theGraph, e);
    }

	// Push the hiddenEdgeStackBottom as a record of how many hidden
	// edges were pushed (also, see above for Contract Edge adjustment)
	sp_Push(theGraph->theStack, hiddenEdgeStackBottom);

	// Moving v's adjacency list to u is aided by knowing the predecessor
	// of u's eBefore (the edge record in u's list before which the
	// edge records of v will be added).
	eBeforePred = gp_IsArc(eBefore)
	              ? gp_GetPrevArc(theGraph, eBefore)
	              : gp_GetLastArc(theGraph, u);

	// Turns out we only need to record six integers related to the edges
	// being moved in order to easily restore them later.
	sp_Push(theGraph->theStack, eBefore);
	sp_Push(theGraph->theStack, gp_GetLastArc(theGraph, v));
	sp_Push(theGraph->theStack, gp_GetFirstArc(theGraph, v));
	sp_Push(theGraph->theStack, eBeforePred);
	sp_Push(theGraph->theStack, u);
	sp_Push(theGraph->theStack, v);

	// For the remaining edge records of v, reassign the 'v' member
	//    of each twin arc to indicate u rather than v.
    e = gp_GetFirstArc(theGraph, v);
    while (gp_IsArc(e))
    {
         gp_SetNeighbor(theGraph, gp_GetTwinArc(theGraph, e), u);
         e = gp_GetNextArc(theGraph, e);
    }

    // If v has any edges left after hiding edges, indicating common neighbors with u, ...
    if (gp_IsArc(gp_GetFirstArc(theGraph, v)))
    {
    	// Then perform the list union of v into u between eBeforePred and eBefore
        if (gp_IsArc(eBeforePred))
        {
        	if (gp_IsArc(gp_GetFirstArc(theGraph, v)))
        	{
            	gp_SetNextArc(theGraph, eBeforePred, gp_GetFirstArc(theGraph, v));
            	gp_SetPrevArc(theGraph, gp_GetFirstArc(theGraph, v), eBeforePred);
        	}
        }
        else
        {
        	gp_SetFirstArc(theGraph, u, gp_GetFirstArc(theGraph, v));
        }

        if (gp_IsArc(eBefore))
        {
        	if (gp_IsArc(gp_GetLastArc(theGraph, v)))
        	{
            	gp_SetNextArc(theGraph, gp_GetLastArc(theGraph, v), eBefore);
            	gp_SetPrevArc(theGraph, eBefore, gp_GetLastArc(theGraph, v));
        	}
        }
        else
        {
        	gp_SetLastArc(theGraph, u, gp_GetLastArc(theGraph, v));
        }

        gp_SetFirstArc(theGraph, v, NIL);
        gp_SetLastArc(theGraph, v, NIL);
    }

    return OK;
}

/********************************************************************
 gp_RestoreVertex()

 This method assumes the built-in graph stack contents are the result
 of vertex hide, vertex identify and edge contract operations.
 This content consists of segments of integers, each segment
 corresponding to the removal of a vertex during an edge contraction
 or vertex identification in which a vertex v was merged into a
 vertex u.  The segment contains two blocks of integers.
 The first block contains information about u, v, the edge records
 in v's adjacency list that were added to u, and where in u's
 adjacency list they were added.  The second block of integers
 contains a list of edges incident to v that were hidden from the
 graph because they were incident to neighbors of v that were also
 neighbors of u (so they would have produced duplicate edges had
 they been left in v's adjacency list when it was merged with u's
 adjacency list).

 This method pops the first block of the segment off the stack and
 uses the information to help remove v's adjacency list from u and
 restore it into v.  Then, the second block is removed from the
 stack, and each indicated edge is restored from the hidden state.

 It is anticipated that this method will be overloaded by extension
 algorithms to perform some processing as each vertex is restored.
 Before restoration, the topmost segment has the following structure:

 ... FHE ... LHE HESB e_u_succ e_v_last e_v_first e_u_pred u v
      ^------------|

 FHE = First hidden edge
 LHE = Last hidden edge
 HESB = Hidden edge stack bottom
 e_u_succ, e_u_pred = The edges of u between which the edges of v
                      were inserted. NIL can appear if the edges of v
                      were added to the beginning or end of u's list
 e_v_first, e_v_last = The first and last edges of v's list, once
                       the hidden edges were removed

 Returns OK for success, NOTOK for internal failure.
 ********************************************************************/

int gp_RestoreVertex(graphP theGraph)
{
	return theGraph->functions.fpRestoreVertex(theGraph);
}

int _RestoreVertex(graphP theGraph)
{
	int u, v, e_u_succ, e_u_pred, e_v_first, e_v_last, HESB, e;

	if (sp_GetCurrentSize(theGraph->theStack) < 7)
    	return NOTOK;

    sp_Pop(theGraph->theStack, v);
    sp_Pop(theGraph->theStack, u);
	sp_Pop(theGraph->theStack, e_u_pred);
	sp_Pop(theGraph->theStack, e_v_first);
	sp_Pop(theGraph->theStack, e_v_last);
	sp_Pop(theGraph->theStack, e_u_succ);

	// If u is not NIL, then vertex v was identified with u.  Otherwise, v was
	// simply hidden, so we skip to restoring the hidden edges.
	if (gp_IsVertex(u))
	{
		// Remove v's adjacency list from u, including accounting for degree 0 case
		if (gp_IsArc(e_u_pred))
		{
			gp_SetNextArc(theGraph, e_u_pred, e_u_succ);
			// If the successor edge exists, link it to the predecessor,
			// otherwise the predecessor is the new last arc
			if (gp_IsArc(e_u_succ))
				gp_SetPrevArc(theGraph, e_u_succ, e_u_pred);
			else
				gp_SetLastArc(theGraph, u, e_u_pred);
		}
		else if (gp_IsArc(e_u_succ))
		{
			// The successor arc exists, but not the predecessor,
			// so the successor is the new first arc
			gp_SetPrevArc(theGraph, e_u_succ, NIL);
			gp_SetFirstArc(theGraph, u, e_u_succ);
		}
		else
		{
			// Just in case u was degree zero
			gp_SetFirstArc(theGraph, u, NIL);
			gp_SetLastArc(theGraph, u, NIL);
		}

		// Place v's adjacency list into v, including accounting for degree 0 case
		gp_SetFirstArc(theGraph, v, e_v_first);
		gp_SetLastArc(theGraph, v, e_v_last);
		if (gp_IsArc(e_v_first))
			gp_SetPrevArc(theGraph, e_v_first, NIL);
		if (gp_IsArc(e_v_last))
			gp_SetPrevArc(theGraph, e_v_last, NIL);

		// For each edge record restored to v's adjacency list, reassign the 'v' member
		//    of each twin arc to indicate v rather than u.
	    e = e_v_first;
	    while (gp_IsArc(e))
	    {
	         gp_SetNeighbor(theGraph, gp_GetTwinArc(theGraph, e), v);
	         e = (e == e_v_last ? NIL : gp_GetNextArc(theGraph, e));
	    }
	}

	// Restore the hidden edges of v, if any
	sp_Pop(theGraph->theStack, HESB);
	return _RestoreHiddenEdges(theGraph, HESB);
}

/********************************************************************
 gp_RestoreVertices()

 This method assumes the built-in graph stack has content consistent
 with numerous vertex identification or edge contraction operations.
 This method unwinds the stack, moving edges back to their original
 vertex owners and restoring hidden edges.
 This method is a simple iterator that invokes gp_RestoreVertex()
 until the stack is empty, so extension algorithms are more likely
 to overload gp_RestoreVertex().

 Returns OK for success, NOTOK for internal failure.
 ********************************************************************/

int gp_RestoreVertices(graphP theGraph)
{
    while (sp_NonEmpty(theGraph->theStack))
    {
    	if (gp_RestoreVertex(theGraph) != OK)
    		return NOTOK;
    }

    return OK;
}