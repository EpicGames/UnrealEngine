/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#include "listcoll.h"

/*****************************************************************************
 The data structure defined by this module manages a set of N objects
 arranged as a collection of circular lists, each containing distinct
 elements from the set.

 On construction, LCNew() creates an array of N nodes, each containing a
 prev and next pointer.  The identity of the node is given by its array index.
 Each node's prev and next pointers are set to NIL, indicating that the node
 is not currently part of a list.  LCReset() can be called to reset all
 pointers to NIL.

 The function LCFree() deallocates the collection of lists and clears the
 pointer variable used to pass the collection.

 An empty list is indicated by NIL.  To begin a list with node I, call
 LCPrepend() or LCAppend() with the NIL list and with I as the node.  The prev
 and next pointers in node I are set to I and I is returned as the head of
 the list.

 Future calls to LCPrepend() add a node J as the new first element of the list,
 so the list given as input is pointed to by J's next, and J is returned as
 the head of the list.

 Future calls to LCAppend() add a node J as the new last element, so the prev
 pointer of the list given as input will indicate node J, and the input list
 is returned as the head of the list.

 LCInsertAfter() adds a node immediately after a given anchor node.

 LCInsertBefore() adds a node immediately before a given anchor node and has
    the same effect on a list as LCPrepend().

 The function LCDelete() removes a node I from a list L.  If node I is in the
 list alone, then its pointers are set to NIL, and NIL is returned as the list.
 If node I is not alone in the list, but it is the head of the list (in other
 words, I is equal to L), then L's sucessor is returned as the new head of the
 list. Whether or not I equals L, node I is deleted by joining its predecessor
 and successor nodes.

 LCCopy() copies the contents of one collection to another if both are of
 equal size.

 LCGetNext() is used for forward iteration through a list in the collection.
 The expected iteration pattern is first to process the node one has, then call
 LCGetNext() to get the next node, so if the result of LCGetNext() would be the
 head of the list, then NIL is returned instead.  This simplifies most
 coding operations involving LCGetNext().

 LCGetPrev() is used for backward iteration through a list in the collection.
 The expected iteration pattern is that the last list element will be obtained
 by an initial call to LCGetPrev() with theNode equal to NIL.  This call
 should appear outside of the iteration loop.  The iteration loop then
 proceeds while the current node is not NIL.  The loop body processes the
 current node, then LCGetPrev() is called with theNode equal to the current
 node.  LCGetPrev() returns NIL if theNode is equal to theList.  Otherwise,
 the predecessor of theNode is returned.

 *****************************************************************************/

/*****************************************************************************
 LCNew()
 *****************************************************************************/

listCollectionP LCNew(int N)
{
listCollectionP theListColl = NULL;

     if (N <= 0) return theListColl;

     theListColl = (listCollectionP) malloc(sizeof(listCollectionRec));
     if (theListColl != NULL)
     {
         theListColl->List = (lcnode *) malloc(N*sizeof(lcnode));
         if (theListColl->List == NULL)
         {
             free(theListColl);
             theListColl = NULL;
         }
         else
         {
             theListColl->N = N;
			 LCReset(theListColl);
         }
     }
     return theListColl;
}

/*****************************************************************************
 LCFree()
 *****************************************************************************/

void LCFree(listCollectionP *pListColl)
{
     if (pListColl==NULL || *pListColl==NULL) return;

     if ((*pListColl)->List != NULL)
         free((*pListColl)->List);

     free(*pListColl);
     *pListColl = NULL;
}

/*****************************************************************************
 LCInsertAfter()
 *****************************************************************************/

void LCInsertAfter(listCollectionP listColl, int theAnchor, int theNewNode)
{
     listColl->List[theNewNode].prev = theAnchor;
     listColl->List[theNewNode].next = listColl->List[theAnchor].next;
     listColl->List[listColl->List[theAnchor].next].prev = theNewNode;
     listColl->List[theAnchor].next = theNewNode;
}

/*****************************************************************************
 LCInsertBefore()
 *****************************************************************************/

void LCInsertBefore(listCollectionP listColl, int theAnchor, int theNewNode)
{
     LCPrepend(listColl, theAnchor, theNewNode);
}