/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#pragma once

#include "CoreMinimal.h"
#include "appconst.h"

typedef struct
{
        int prev, next;
} lcnode;

typedef struct
{
        int N;
        lcnode *List;
} listCollectionRec;

typedef listCollectionRec * listCollectionP;

listCollectionP LCNew(int N);
void LCFree(listCollectionP *pListColl);

void LCInsertAfter(listCollectionP listColl, int theAnchor, int theNewNode);
void LCInsertBefore(listCollectionP listColl, int theAnchor, int theNewNode);

/*****************************************************************************
 LCReset()
 *****************************************************************************/

inline void LCReset(listCollectionP listColl)
{
int  K;

     for (K=0; K < listColl->N; K++)
          listColl->List[K].prev = listColl->List[K].next = NIL;
}

/*****************************************************************************
 LCCopy()
 *****************************************************************************/

inline void LCCopy(listCollectionP dst, listCollectionP src)
{
int  K;

     if (dst==NULL || src==NULL || dst->N != src->N) return;

     for (K=0; K < dst->N; K++)
          dst->List[K] = src->List[K];

}

/*****************************************************************************
 LCGetNext()
 *****************************************************************************/

inline int  LCGetNext(listCollectionP listColl, int theList, int theNode)
{
int  next;

     if (listColl==NULL || theList==NIL || theNode==NIL) return NIL;
     next = listColl->List[theNode].next;
     return next==theList ? NIL : next;
}

/*****************************************************************************
 LCGetPrev()
 *****************************************************************************/

inline int  LCGetPrev(listCollectionP listColl, int theList, int theNode)
{
     if (listColl==NULL || theList==NIL) return NIL;
     if (theNode == NIL) return listColl->List[theList].prev;
     if (theNode == theList) return NIL;
     return listColl->List[theNode].prev;
}

/*****************************************************************************
 LCAppend()
 *****************************************************************************/

inline int  LCAppend(listCollectionP listColl, int theList, int theNode)
{
     /* If the given list is empty, then the given node becomes the
        singleton list output */

     if (theList == NIL)
     {
         listColl->List[theNode].prev = listColl->List[theNode].next = theNode;
         theList = theNode;
     }

     /* Otherwise, make theNode the predecessor of head node of theList,
        which is where the last node goes in a circular list. */

     else
     {
     int pred = listColl->List[theList].prev;

         listColl->List[theList].prev = theNode;
         listColl->List[theNode].next = theList;
         listColl->List[theNode].prev = pred;
         listColl->List[pred].next = theNode;
     }

     /* Return the list (only really important if it was NIL) */

     return theList;
}

/*****************************************************************************
 LCPrepend()
 *****************************************************************************/

inline int  LCPrepend(listCollectionP listColl, int theList, int theNode)
{
     /* If the append worked, then theNode is last, which in a circular
        list is the direct predecessor of the list head node, so we
        just back up one. For singletons, the result is unchanged. */

     return listColl->List[LCAppend(listColl, theList, theNode)].prev;
}

/*****************************************************************************
 LCDelete()
 *****************************************************************************/

inline int  LCDelete(listCollectionP listColl, int theList, int theNode)
{
     /* If the list is a singleton, then NIL its pointers and
        return NIL for theList*/

     if (listColl->List[theList].next == theList)
     {
         listColl->List[theList].prev = listColl->List[theList].next = NIL;
         theList = NIL;
     }

     /* Join predecessor and successor, dropping theNode from the list.
        If theNode is the head of the list, then return the successor as
        the new head node. */

     else
     {
     int pred=listColl->List[theNode].prev,
         succ=listColl->List[theNode].next;

         listColl->List[pred].next = succ;
         listColl->List[succ].prev = pred;

         listColl->List[theNode].prev = listColl->List[theNode].next = NIL;

         if (theList == theNode)
             theList = succ;
     }

     return theList;
}