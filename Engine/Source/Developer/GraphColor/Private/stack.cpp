/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#include "stack.h"
#include "CoreMinimal.h"

stackP sp_New(int capacity)
{
stackP theStack;

     theStack = (stackP) malloc(sizeof(stack));

     if (theStack != NULL)
     {
         theStack->S = (int *) malloc(capacity*sizeof(int));
         if (theStack->S == NULL)
         {
             free(theStack);
             theStack = NULL;
         }
     }

     if (theStack != NULL)
     {
         theStack->capacity = capacity;
         sp_ClearStack(theStack);
     }

     return theStack;
}

void sp_Free(stackP *pStack)
{
     if (pStack == NULL || *pStack == NULL) return;

     (*pStack)->capacity = (*pStack)->size = 0;

     if ((*pStack)->S != NULL)
          free((*pStack)->S);
     (*pStack)->S = NULL;
     free(*pStack);

     *pStack = NULL;
}

int  sp_CopyContent(stackP stackDst, stackP stackSrc)
{
     if (stackDst->capacity < stackSrc->size)
         return NOTOK;

     if (stackSrc->size > 0)
         memcpy(stackDst->S, stackSrc->S, stackSrc->size*sizeof(int));

     stackDst->size = stackSrc->size;
     return OK;
}

stackP sp_Duplicate(stackP theStack)
{
stackP newStack = sp_New(theStack->capacity);

    if (newStack == NULL)
        return NULL;

    if (theStack->size > 0)
    {
        memcpy(newStack->S, theStack->S, theStack->size*sizeof(int));
        newStack->size = theStack->size;
    }

    return newStack;
}

int  sp_Copy(stackP stackDst, stackP stackSrc)
{
    if (sp_CopyContent(stackDst, stackSrc) != OK)
    {
    stackP newStack = sp_Duplicate(stackSrc);
    int  *p;

         if (newStack == NULL)
             return NOTOK;

         p = stackDst->S;
         stackDst->S = newStack->S;
         newStack->S = p;
         newStack->capacity = stackDst->capacity;
         sp_Free(&newStack);

         stackDst->size = stackSrc->size;
         stackDst->capacity = stackSrc->capacity;
    }

    return OK;
}