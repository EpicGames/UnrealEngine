/*
Planarity-Related Graph Algorithms Project
Copyright (c) 1997-2012, John M. Boyer
All rights reserved. Includes a reference implementation of the following:

* John M. Boyer. "Subgraph Homeomorphism via the Edge Addition Planarity Algorithm".
  Journal of Graph Algorithms and Applications, Vol. 16, no. 2, pp. 381-410, 2012.
  http://www.jgaa.info/16/268.html

* John M. Boyer. "A New Method for Efficiently Generating Planar Graph
  Visibility Representations". In P. Eades and P. Healy, editors,
  Proceedings of the 13th International Conference on Graph Drawing 2005,
  Lecture Notes Comput. Sci., Volume 3843, pp. 508-511, Springer-Verlag, 2006.

* John M. Boyer and Wendy J. Myrvold. "On the Cutting Edge: Simplified O(n)
  Planarity by Edge Addition". Journal of Graph Algorithms and Applications,
  Vol. 8, No. 3, pp. 241-273, 2004.
  http://www.jgaa.info/08/91.html

* John M. Boyer. "Simplified O(n) Algorithms for Planar Graph Embedding,
  Kuratowski Subgraph Isolation, and Related Problems". Ph.D. Dissertation,
  University of Victoria, 2001.

Redistribution and use in source and binary forms, with or without modification,
are permitted provided that the following conditions are met:

* Redistributions of source code must retain the above copyright notice, this
  list of conditions and the following disclaimer.

* Redistributions in binary form must reproduce the above copyright notice, this
  list of conditions and the following disclaimer in the documentation and/or
  other materials provided with the distribution.

* Neither the name of the Planarity-Related Graph Algorithms Project nor the names
  of its contributors may be used to endorse or promote products derived from this
  software without specific prior written permission.

THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT OWNER OR CONTRIBUTORS BE LIABLE FOR
ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL DAMAGES
(INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR SERVICES;
LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER CAUSED AND ON
ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY, OR TORT
(INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE OF THIS
SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.
*/

#pragma once

#include "CoreMinimal.h"
#include "appconst.h"

typedef struct
{
        int *S;
        int size, capacity;
} stack;

typedef stack * stackP;

stackP sp_New(int);
void sp_Free(stackP *);

int  sp_Copy(stackP, stackP);

int  sp_CopyContent(stackP stackDst, stackP stackSrc);
stackP sp_Duplicate(stackP theStack);

#define sp_GetCapacity(theStack) (theStack->capacity)

#define sp_Push(theStack, a) { if (sp__Push(theStack, (a)) != OK) return NOTOK; }
#define sp_Push2(theStack, a, b) { if (sp__Push2(theStack, (a), (b)) != OK) return NOTOK; }

#define sp_Pop(theStack, a) { if (sp__Pop(theStack, &(a)) != OK) return NOTOK; }
#define sp_Pop2(theStack, a, b) { if (sp__Pop2(theStack, &(a), &(b)) != OK) return NOTOK; }

inline int  sp_ClearStack(stackP theStack)
{
     theStack->size = 0;
     return OK;
}

inline int  sp_GetCurrentSize(stackP theStack)
{
     return theStack->size;
}

inline int  sp_SetCurrentSize(stackP theStack, int size)
{
	 return size > theStack->capacity ? NOTOK : (theStack->size = size, OK);
}

inline int  sp_IsEmpty(stackP theStack)
{
     return !theStack->size;
}

inline int  sp_NonEmpty(stackP theStack)
{
     return theStack->size;
}

inline int  sp__Push(stackP theStack, int a)
{
     if (theStack->size >= theStack->capacity)
         return NOTOK;

     theStack->S[theStack->size++] = a;
     return OK;
}

inline int  sp__Push2(stackP theStack, int a, int b)
{
     if (theStack->size + 1 >= theStack->capacity)
         return NOTOK;

     theStack->S[theStack->size++] = a;
     theStack->S[theStack->size++] = b;
     return OK;
}

inline int  sp__Pop(stackP theStack, int *pA)
{
     if (theStack->size <= 0)
         return NOTOK;

     *pA = theStack->S[--theStack->size];
     return OK;
}

inline int  sp__Pop2(stackP theStack, int *pA, int *pB)
{
     if (theStack->size <= 1)
         return NOTOK;

     *pB = theStack->S[--theStack->size];
     *pA = theStack->S[--theStack->size];

     return OK;
}

inline int  sp_Top(stackP theStack)
{
    return theStack->size ? theStack->S[theStack->size-1] : NIL;
}

inline int  sp_Get(stackP theStack, int pos)
{
	 if (theStack == NULL || pos < 0 || pos >= theStack->size)
		 return NOTOK;

     return (theStack->S[pos]);
}

inline int  sp_Set(stackP theStack, int pos, int val)
{
	 if (theStack == NULL || pos < 0 || pos >= theStack->size)
		 return NOTOK;

	 return (theStack->S[pos] = val);
}