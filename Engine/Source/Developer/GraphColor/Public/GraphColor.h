// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

GRAPHCOLOR_API void GraphColorMesh( TArray< uint8 >& VertColors, const TArray< uint32 >& Indexes );